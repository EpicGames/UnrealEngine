// Copyright Epic Games, Inc. All Rights Reserved.

#include "JsonTraceAnalyzer.h"

#include "Containers/ArrayView.h"

// Public TraceServices utilities (LegacyAttachmentArray, LegacyAttachmentString)
#include "TraceServices/Utils.h"

#include <stdio.h>

////////////////////////////////////////////////////////////////////////////////////////////////////
// Decode utilities  --  duplicated from TraceServices/Private/Common/Utils.h
// These are private to TraceServices, so we inline them here (same approach as
// ConvertToText.cpp in TraceAnalyzer).
// Only needed for Stats batch decode (CpuProfiler and Counters now use TraceServices analyzers).
////////////////////////////////////////////////////////////////////////////////////////////////////

namespace TraceQueryLocal
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// Minimal CBOR primitive decoders  --  used to extract integer fields from CPU profiler metadata
// payloads (e.g. "Csv Frame Number" from the RHI Frame breadcrumb).
// Only handles major type 0 (unsigned integer)  --  sufficient for frame counter fields.
////////////////////////////////////////////////////////////////////////////////////////////////////

// Advance Offset past one CBOR value header + payload.  Works for any major type whose
// "additional info" encodes a scalar length (covers all integer, float, and simple types).
// Does NOT recurse into containers.  Returns false if the buffer is truncated.
static bool SkipCborPrimitive(const uint8* Data, int32 Len, int32& Offset)
{
	if (Offset >= Len)
	{
		return false;
	}
	uint8 Info = Data[Offset++] & 0x1F;
	if (Info < 24)
	{
		return true;
	}
	if (Info == 24 && Offset + 1 <= Len)
	{
		Offset += 1;
		return true;
	}
	if (Info == 25 && Offset + 2 <= Len)
	{
		Offset += 2;
		return true;
	}
	if (Info == 26 && Offset + 4 <= Len)
	{
		Offset += 4;
		return true;
	}
	if (Info == 27 && Offset + 8 <= Len)
	{
		Offset += 8;
		return true;
	}
	return false;
}

// Read a CBOR unsigned integer (major type 0) at Offset and advance past it.
// Returns false if the next value is not a uint or the buffer is truncated.
static bool DecodeCborUint(const uint8* Data, int32 Len, int32& Offset, uint64& OutValue)
{
	if (Offset >= Len)
	{
		return false;
	}
	uint8 InitByte = Data[Offset++];
	if ((InitByte >> 5) != 0)
	{
		return false;  // Not major type 0 (uint)
	}
	uint8 Info = InitByte & 0x1F;
	if (Info < 24)
	{
		OutValue = Info;
		return true;
	}
	if (Info == 24 && Offset + 1 <= Len)
	{
		OutValue = Data[Offset++];
		return true;
	}
	if (Info == 25 && Offset + 2 <= Len)
	{
		OutValue = (uint64(Data[Offset]) << 8) | Data[Offset + 1];
		Offset += 2;
		return true;
	}
	if (Info == 26 && Offset + 4 <= Len)
	{
		OutValue = (uint64(Data[Offset]) << 24) | (uint64(Data[Offset + 1]) << 16) |
		           (uint64(Data[Offset + 2]) << 8) | Data[Offset + 3];
		Offset += 4;
		return true;
	}
	if (Info == 27 && Offset + 8 <= Len)
	{
		OutValue = 0;
		for (int32 i = 0; i < 8; ++i)
		{
			OutValue = (OutValue << 8) | Data[Offset + i];
		}
		Offset += 8;
		return true;
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

inline uint64 Decode7bit(const uint8*& BufferPtr)
{
	uint64 Value = 0;
	uint64 ByteIndex = 0;
	bool bHasMoreBytes;
	do
	{
		uint8 ByteValue = *BufferPtr++;
		bHasMoreBytes = ByteValue & 0x80;
		Value |= uint64(ByteValue & 0x7f) << (ByteIndex * 7);
		++ByteIndex;
	} while (bHasMoreBytes);
	return Value;
}

inline int64 DecodeZigZag(const uint8*& BufferPtr)
{
	uint64 Z = Decode7bit(BufferPtr);
	return (Z & 1) ? (Z >> 1) ^ -1 : (Z >> 1);
}

inline uint32 GetThreadIdField(
	const UE::Trace::IAnalyzer::FOnEventContext& Context,
	const ANSICHAR* FieldName = "ThreadId")
{
	static const uint32 Bias = 0x70000000;
	uint32 ThreadId = Context.EventData.GetValue<uint32>(FieldName, 0);
	ThreadId |= ThreadId ? Bias : Context.ThreadInfo.GetId();
	return ThreadId;
}

} // namespace TraceQueryLocal

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTraceQueryCpuProfilerProvider
////////////////////////////////////////////////////////////////////////////////////////////////////

const FString* FTraceQueryCpuProfilerProvider::LookupTimerName(uint32 TimerId) const
{
	// Handle metadata bit-inversion: negative int32 means metadata entry
	if (int32(TimerId) < 0)
	{
		uint32 MetaIdx = ~TimerId;
		if (MetaIdx < uint32(Metadatas.Num()))
		{
			TimerId = Metadatas[MetaIdx].TimerId;
		}
		else
		{
			return nullptr;
		}
	}

	if (TimerId < uint32(TimerNames.Num()) && TimerNames[TimerId].IsSet())
	{
		return &TimerNames[TimerId].GetValue();
	}
	return nullptr;
}

const FString* FTraceQueryCpuProfilerProvider::LookupThreadName(uint32 ThreadId) const
{
	return ThreadNames.Find(ThreadId);
}

void FTraceQueryCpuProfilerProvider::SetThreadName(uint32 ThreadId, const FString& Name)
{
	ThreadNames.Add(ThreadId, Name);
}

void FTraceQueryCpuProfilerProvider::AddThread(uint32 Id, const TCHAR* Name, EThreadPriority Priority)
{
	// Note: Name is always nullptr here  --  the CpuProfiler analyzer calls AddThread for fault
	// tolerance but doesn't have thread names. Real names arrive via FJsonTraceAnalyzer::OnThreadInfo.
	if (!Threads.Contains(Id))
	{
		Threads.Add(Id, MakeUnique<FThread>(Id, this));
	}
}

uint32 FTraceQueryCpuProfilerProvider::AddTimer(TraceServices::ETimingProfilerTimerType Type)
{
	if (Type == TraceServices::ETimingProfilerTimerType::CPU)
	{
		return AddTimerInternal(FStringView());
	}
	return 0;
}

uint32 FTraceQueryCpuProfilerProvider::AddCpuTimer(FStringView Name, const TCHAR* File, uint32 Line)
{
	return AddTimerInternal(Name);
}

uint32 FTraceQueryCpuProfilerProvider::AddTimerInternal(FStringView Name)
{
	TOptional<FString> TimerName;
	if (!Name.IsEmpty())
	{
		TimerName.Emplace(FString(Name));
	}

	uint32 TimerId = TimerNames.Add(TimerName);

	if (NumCpuSpecs)
	{
		++(*NumCpuSpecs);
	}

	return TimerId;
}

void FTraceQueryCpuProfilerProvider::SetTimerName(uint32 TimerId, FStringView Name)
{
	check(TimerId < uint32(TimerNames.Num()));
	TimerNames[TimerId].Emplace(FString(Name));
}

uint32 FTraceQueryCpuProfilerProvider::AddMetadata(uint32 MainTimerId, TArray<uint8>&& Metadata)
{
	uint32 MetadataId = Metadatas.Num();
	Metadatas.Add({ MoveTemp(Metadata), MainTimerId });
	return ~MetadataId;
}

TArrayView<uint8> FTraceQueryCpuProfilerProvider::GetEditableMetadata(uint32 TimerId)
{
	if (int32(TimerId) >= 0)
	{
		return TArrayView<uint8>();
	}

	uint32 MetaIdx = ~TimerId;
	if (MetaIdx >= uint32(Metadatas.Num()))
	{
		return TArrayView<uint8>();
	}

	return Metadatas[MetaIdx].Payload;
}

TraceServices::IEditableTimeline<TraceServices::FTimingProfilerEvent>& FTraceQueryCpuProfilerProvider::GetCpuThreadEditableTimeline(uint32 ThreadId)
{
	TUniquePtr<FThread>* Found = Threads.Find(ThreadId);
	if (Found)
	{
		return *(Found->Get());
	}
	return *Threads.Add(ThreadId, MakeUnique<FThread>(ThreadId, this));
}

void FTraceQueryCpuProfilerProvider::FThread::AppendBeginEvent(double StartTime, const TraceServices::FTimingProfilerEvent& Event)
{
	FScopeEnter Entry;
	Entry.TimerId = Event.TimerIndex;
	Entry.StartTime = StartTime;
	ScopeStack.Push(Entry);
}

void FTraceQueryCpuProfilerProvider::FThread::AppendEndEvent(double EndTime)
{
	if (ScopeStack.IsEmpty())
	{
		return;
	}

	FScopeEnter Entry = ScopeStack.Pop();

	if (Provider->NumCpuEvents)
	{
		++(*Provider->NumCpuEvents);
	}

	if (Provider->Options && Provider->Options->bSummaryOnly)
	{
		return;
	}

	double Duration = EndTime - Entry.StartTime;

	if (Provider->Options)
	{
		if (Provider->Options->TimeA >= 0.0 && Entry.StartTime < Provider->Options->TimeA)
		{
			return;
		}
		if (Provider->Options->TimeB >= 0.0 && Entry.StartTime > Provider->Options->TimeB)
		{
			return;
		}
	}

	// Resolve timer name  --  use the main timer ID for metadata entries
	uint32 ResolvedTimerId = Entry.TimerId;
	if (int32(Entry.TimerId) < 0)
	{
		uint32 MetaIdx = ~Entry.TimerId;
		if (MetaIdx < uint32(Provider->Metadatas.Num()))
		{
			ResolvedTimerId = Provider->Metadatas[MetaIdx].TimerId;
		}
	}

	// Apply timer name filter
	if (Provider->Options && !Provider->Options->TimerFilter.IsEmpty())
	{
		const FString* TimerName = Provider->LookupTimerName(Entry.TimerId);
		if (!TimerName || !TimerName->Contains(Provider->Options->TimerFilter))
		{
			return;
		}
	}

	// CSV frame number extraction (CsvFrames logger mode).
	// When bWantCsvFrameNumbers is set, only "Frame" scopes with breadcrumb metadata are
	// captured.  All other scopes are dropped  --  we don't need full scope group output here.
	if (Provider->Options && Provider->Options->bWantCsvFrameNumbers)
	{
		if (int32(Entry.TimerId) < 0)
		{
			const FString* TimerName = Provider->LookupTimerName(Entry.TimerId);
			if (TimerName && *TimerName == TEXT("Frame") && Provider->CsvFrameNumbers)
			{
				uint32 MetaIdx = ~Entry.TimerId;
				if (MetaIdx < uint32(Provider->Metadatas.Num()))
				{
					const TArray<uint8>& Payload = Provider->Metadatas[MetaIdx].Payload;
					// The RHI Frame breadcrumb payload is two sequential CBOR uints:
					//   Field 0: "Frame Number"      (CurrentFrameCounter)
					//   Field 1: "Csv Frame Number"  (FCsvProfiler::GetCaptureFrameNumberRT())
					// Skip the first value, then read the second.
					int32 ReadOffset = 0;
					uint64 CsvFrameNum = 0;
					if (TraceQueryLocal::SkipCborPrimitive(Payload.GetData(), Payload.Num(), ReadOffset) &&
					    TraceQueryLocal::DecodeCborUint(Payload.GetData(), Payload.Num(), ReadOffset, CsvFrameNum))
					{
						Provider->CsvFrameNumbers->Add({ Entry.StartTime, (int64)CsvFrameNum });
					}
				}
			}
		}
		return;  // In CsvFrames mode, never add to ScopeGroups
	}

	if (Provider->ScopeGroups)
	{
		FScopeGroupKey Key;
		Key.ThreadId = ThreadId;
		Key.SpecId = ResolvedTimerId;

		// Resolve parent from the scope stack (after pop, top is the parent)
		uint32 ParentSpecId = 0;
		if (!ScopeStack.IsEmpty())
		{
			uint32 ParentTimerId = ScopeStack.Last().TimerId;
			if (int32(ParentTimerId) < 0)
			{
				uint32 MetaIdx = ~ParentTimerId;
				if (MetaIdx < uint32(Provider->Metadatas.Num()))
				{
					ParentSpecId = Provider->Metadatas[MetaIdx].TimerId;
				}
			}
			else
			{
				ParentSpecId = ParentTimerId;
			}
		}

		FScopeDataPoint Point;
		Point.Time = Entry.StartTime;
		Point.Duration = Duration;
		Point.Depth = ScopeStack.Num();
		Point.ParentSpecId = ParentSpecId;
		Provider->ScopeGroups->FindOrAdd(Key).Add(Point);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTraceQueryCounter + FTraceQueryCounterProvider
////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceQueryCounter::SetValue(double Time, int64 Value)
{
	if (NumCounterValues)
	{
		++(*NumCounterValues);
	}

	if (!Options || Options->bSummaryOnly)
	{
		return;
	}

	if (TimeFilterCallback && !TimeFilterCallback(Time))
	{
		return;
	}

	if (Options && !Options->CounterFilter.IsEmpty() && !Name.Contains(Options->CounterFilter))
	{
		return;
	}

	if (EmitCallback && JsonEscapeCallback)
	{
		FString Line = FString::Printf(
			TEXT("{\"type\":\"counter\",\"time\":%.6f,\"counterId\":%u,\"name\":\"%s\",\"value\":%lld}"),
			Time, (uint32)CounterId, *JsonEscapeCallback(Name), Value);
		EmitCallback(TCHAR_TO_UTF8(*Line));
	}
}

void FTraceQueryCounter::SetValue(double Time, double Value)
{
	if (NumCounterValues)
	{
		++(*NumCounterValues);
	}

	if (!Options || Options->bSummaryOnly)
	{
		return;
	}

	if (TimeFilterCallback && !TimeFilterCallback(Time))
	{
		return;
	}

	if (Options && !Options->CounterFilter.IsEmpty() && !Name.Contains(Options->CounterFilter))
	{
		return;
	}

	if (EmitCallback && JsonEscapeCallback)
	{
		FString Line = FString::Printf(
			TEXT("{\"type\":\"counter\",\"time\":%.6f,\"counterId\":%u,\"name\":\"%s\",\"value\":%.9g}"),
			Time, (uint32)CounterId, *JsonEscapeCallback(Name), Value);
		EmitCallback(TCHAR_TO_UTF8(*Line));
	}
}

TraceServices::IEditableCounter* FTraceQueryCounterProvider::CreateEditableCounter()
{
	auto Counter = MakeUnique<FTraceQueryCounter>();
	Counter->Options = Options;
	Counter->EmitCallback = EmitCallback;
	Counter->TimeFilterCallback = TimeFilterCallback;
	Counter->JsonEscapeCallback = JsonEscapeCallback;
	Counter->NumCounterValues = NumCounterValues;
	Counter->CounterId = NextCounterId++;

	if (NumCounterSpecs)
	{
		++(*NumCounterSpecs);
	}

	FTraceQueryCounter* Ptr = Counter.Get();
	Counters.Add(MoveTemp(Counter));
	return Ptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FJsonTraceAnalyzer
////////////////////////////////////////////////////////////////////////////////////////////////////

FJsonTraceAnalyzer::FJsonTraceAnalyzer(const FTraceQueryOptions& InOptions)
	: Options(InOptions)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FJsonTraceAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	FInterfaceBuilder& Builder = Context.InterfaceBuilder;

	// Always subscribe to diagnostics and frame events for metadata/structure
	Builder.RouteEvent(RouteId_DiagSession, "Diagnostics", "Session");
	Builder.RouteEvent(RouteId_DiagSession2, "Diagnostics", "Session2");
	Builder.RouteEvent(RouteId_BeginFrame, "Misc", "BeginFrame");

	if (Options.bWantAllEvents)
	{
		Builder.RouteAllEvents(RouteId_AllEvents);
	}

	if (Options.bWantStats || Options.bWantAllEvents)
	{
		Builder.RouteEvent(RouteId_StatsSpec, "Stats", "Spec");
		Builder.RouteEvent(RouteId_StatsEventBatch, "Stats", "EventBatch");
		Builder.RouteEvent(RouteId_StatsEventBatch2, "Stats", "EventBatch2");
	}

	// Note: CpuProfiler and Counters routes are registered by the factory-created analyzers,
	// not by this analyzer. See TraceQuery.cpp for the wiring.
}

void FJsonTraceAnalyzer::OnThreadInfo(const FThreadInfo& ThreadInfo)
{
	// Feed thread names from the trace's Diagnostics.Thread events into our CpuProfiler provider
	if (CpuProfilerProvider)
	{
		uint32 ThreadId = ThreadInfo.GetId();
		FString Name = ThreadInfo.GetName();
		if (!Name.IsEmpty())
		{
			CpuProfilerProvider->SetThreadName(ThreadId, Name);
		}
	}
}

void FJsonTraceAnalyzer::OnAnalysisEnd()
{
	// Flush buffered frame groups as grouped JSONL
	static const char* FrameTypeNames[] = { "game", "render" };
	for (TPair<uint8, TArray<double>>& Pair : FrameGroups)
	{
		uint8 FrameType = Pair.Key;
		TArray<double>& Times = Pair.Value;
		const char* FrameTypeName = (FrameType < UE_ARRAY_COUNT(FrameTypeNames)) ? FrameTypeNames[FrameType] : "unknown";

		FString DataArray;
		DataArray.Reserve(Times.Num() * 16);
		DataArray += TEXT("[");
		for (int32 Index = 0; Index < Times.Num(); ++Index)
		{
			if (Index > 0)
			{
				DataArray += TEXT(",");
			}
			DataArray += FString::Printf(TEXT("%.6f"), Times[Index]);
		}
		DataArray += TEXT("]");

		FString Line = FString::Printf(
			TEXT("{\"type\":\"frame\",\"frameType\":\"%s\",\"count\":%d,\"data\":%s}"),
			ANSI_TO_TCHAR(FrameTypeName), Times.Num(), *DataArray);
		EmitJsonLine(TCHAR_TO_UTF8(*Line));
	}

	// Emit CSV frame number records (populated when bWantCsvFrameNumbers is set).
	// One record per game frame that had a "Frame" scope with RHI breadcrumb metadata.
	// {"type":"csv_frame_number","time_s":N.NNN,"csv_frame":NNN}
	for (const TPair<double, int64>& Pair : CsvFrameNumbers)
	{
		FString Line = FString::Printf(
			TEXT("{\"type\":\"csv_frame_number\",\"time_s\":%.6f,\"csv_frame\":%lld}"),
			Pair.Key, Pair.Value);
		EmitJsonLine(TCHAR_TO_UTF8(*Line));
	}

	// Flush buffered CPU scope groups as grouped JSONL
	for (TPair<FScopeGroupKey, TArray<FScopeDataPoint>>& Pair : ScopeGroups)
	{
		const FScopeGroupKey& Key = Pair.Key;
		TArray<FScopeDataPoint>& Points = Pair.Value;

		// Resolve timer name via the CpuProfiler provider
		FString TimerName;
		if (CpuProfilerProvider)
		{
			const FString* Name = CpuProfilerProvider->LookupTimerName(Key.SpecId);
			TimerName = Name ? *Name : FString::Printf(TEXT("Timer_%u"), Key.SpecId);
		}
		else
		{
			TimerName = FString::Printf(TEXT("Timer_%u"), Key.SpecId);
		}

		// Resolve parent names for this group's data points
		// Build data array: [[time,dur,depth,parentName],...]
		FString DataArray;
		DataArray.Reserve(Points.Num() * 48);
		DataArray += TEXT("[");
		for (int32 Index = 0; Index < Points.Num(); ++Index)
		{
			if (Index > 0)
			{
				DataArray += TEXT(",");
			}

			FString ParentName;
			if (Points[Index].ParentSpecId != 0 && CpuProfilerProvider)
			{
				const FString* Name = CpuProfilerProvider->LookupTimerName(Points[Index].ParentSpecId);
				if (Name)
				{
					ParentName = *Name;
				}
			}

			if (ParentName.IsEmpty())
			{
				DataArray += FString::Printf(TEXT("[%.6f,%.9g,%d,null]"),
					Points[Index].Time, Points[Index].Duration, Points[Index].Depth);
			}
			else
			{
				DataArray += FString::Printf(TEXT("[%.6f,%.9g,%d,\"%s\"]"),
					Points[Index].Time, Points[Index].Duration, Points[Index].Depth,
					*JsonEscape(ParentName));
			}
		}
		DataArray += TEXT("]");

		// Resolve thread name via the CpuProfiler provider
		FString ThreadName;
		if (CpuProfilerProvider)
		{
			const FString* Name = CpuProfilerProvider->LookupThreadName(Key.ThreadId);
			ThreadName = Name ? *Name : FString::Printf(TEXT("Thread_%u"), Key.ThreadId);
		}
		else
		{
			ThreadName = FString::Printf(TEXT("Thread_%u"), Key.ThreadId);
		}

		FString Line = FString::Printf(
			TEXT("{\"type\":\"cpu_scope\",\"thread\":%u,\"threadName\":\"%s\",\"specId\":%u,\"name\":\"%s\",\"count\":%d,\"data\":%s}"),
			Key.ThreadId, *JsonEscape(ThreadName), Key.SpecId, *JsonEscape(TimerName), Points.Num(), *DataArray);
		EmitJsonLine(TCHAR_TO_UTF8(*Line));
	}

	if (Options.bSummaryOnly)
	{
		fprintf(stdout, "=== TraceQuery Summary ===\n");
		fprintf(stdout, "Stats specs:      %llu\n", NumStatsSpecs);
		fprintf(stdout, "Stats batches:    %llu\n", NumStatsBatches);
		fprintf(stdout, "Stats operations: %llu\n", NumStatsOps);
		fprintf(stdout, "Counter specs:    %llu\n", NumCounterSpecs);
		fprintf(stdout, "Counter values:   %llu\n", NumCounterValues);
		fprintf(stdout, "Frames:           %llu\n", NumFrames);
		fprintf(stdout, "CPU timer specs:  %llu\n", NumCpuSpecs);
		fprintf(stdout, "CPU events:       %llu\n", NumCpuEvents);
		fprintf(stdout, "Total events:     %llu\n", NumTotalEvents);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FJsonTraceAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
{
	++NumTotalEvents;

	switch (RouteId)
	{
	case RouteId_StatsSpec:
		OnStatsSpec(Context);
		break;
	case RouteId_StatsEventBatch:
		OnStatsEventBatch(Context, /*bV2=*/false);
		break;
	case RouteId_StatsEventBatch2:
		OnStatsEventBatch(Context, /*bV2=*/true);
		break;
	case RouteId_DiagSession:
		// Session v1 uses attachment-based parsing; Session2 is cleaner.
		// For v1, extract what we can.
		break;
	case RouteId_DiagSession2:
		OnDiagnosticsSession2(Context);
		break;
	case RouteId_BeginFrame:
		OnBeginFrame(Context);
		break;
	default:
		break;
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Stats
////////////////////////////////////////////////////////////////////////////////////////////////////

void FJsonTraceAnalyzer::OnStatsSpec(const FOnEventContext& Context)
{
	const FEventData& EventData = Context.EventData;
	uint32 StatId = EventData.GetValue<uint32>("Id");

	FStatSpec& Spec = StatSpecs.FindOrAdd(StatId);

	if (!EventData.GetString("Name", Spec.Name))
	{
		// Legacy: name in attachment
		const uint8* Attachment = EventData.GetAttachment();
		if (Attachment)
		{
			Spec.Name = FString(reinterpret_cast<const ANSICHAR*>(Attachment));
		}
	}
	EventData.GetString("Group", Spec.Group);
	EventData.GetString("Description", Spec.Description);
	Spec.bIsFloatingPoint = EventData.GetValue<bool>("IsFloatingPoint");
	Spec.bIsResetEveryFrame = EventData.GetValue<bool>("ShouldClearEveryFrame");

	++NumStatsSpecs;
}

void FJsonTraceAnalyzer::OnStatsEventBatch(const FOnEventContext& Context, bool bV2)
{
	enum class EOpType : uint8
	{
		Increment = 0,
		Decrement = 1,
		AddInteger = 2,
		SetInteger = 3,
		AddFloat = 4,
		SetFloat = 5,
	};

	++NumStatsBatches;

	if (Options.bSummaryOnly)
	{
		// Still need to count operations for summary
		TArrayView<const uint8> DataView = TraceServices::FTraceAnalyzerUtils::LegacyAttachmentArray("Data", Context);
		const uint8* BufferPtr = DataView.GetData();
		const uint8* BufferEnd = BufferPtr + DataView.Num();
		while (BufferPtr < BufferEnd)
		{
			uint64 DecodedIdAndOp = TraceQueryLocal::Decode7bit(BufferPtr);
			uint8 Op = DecodedIdAndOp & 0x7;
			TraceQueryLocal::Decode7bit(BufferPtr); // CycleDiff
			// Skip value
			EOpType SummaryOp = static_cast<EOpType>(Op);
			if (SummaryOp == EOpType::AddInteger || SummaryOp == EOpType::SetInteger)
			{
				TraceQueryLocal::DecodeZigZag(BufferPtr);
			}
			else if (SummaryOp == EOpType::AddFloat || SummaryOp == EOpType::SetFloat)
			{
				BufferPtr += sizeof(double);
			}
			// Op 0 (Inc) and 1 (Dec) have no value payload
			++NumStatsOps;
		}
		if (BufferPtr != BufferEnd)
		{
			UE_LOGF(LogTraceQuery, Warning, "Stats batch buffer mismatch (summary): %lld bytes remaining", (int64)(BufferEnd - BufferPtr));
		}
		return;
	}

	// Decode logic ported from StatsTraceAnalysis.cpp:189-274
	uint32 ThreadId = TraceQueryLocal::GetThreadIdField(Context);
	FThreadState& ThreadState = GetThreadState(ThreadId);

	const uint64 BaseTimestamp = Context.EventTime.AsCycle64() - Context.EventTime.GetTimestamp();

	TArrayView<const uint8> DataView = TraceServices::FTraceAnalyzerUtils::LegacyAttachmentArray("Data", Context);
	const uint8* BufferPtr = DataView.GetData();
	const uint8* BufferEnd = BufferPtr + DataView.Num();

	while (BufferPtr < BufferEnd)
	{
		uint64 DecodedIdAndOp = TraceQueryLocal::Decode7bit(BufferPtr);
		uint32 StatId = static_cast<uint32>(DecodedIdAndOp >> 3);
		uint8 Op = DecodedIdAndOp & 0x7;
		uint64 CycleDiff = TraceQueryLocal::Decode7bit(BufferPtr);

		if (bV2)
		{
			if (CycleDiff >= BaseTimestamp)
			{
				ThreadState.LastCycle = 0;
			}
		}

		uint64 Cycle = ThreadState.LastCycle + CycleDiff;
		double Time = Context.EventTime.AsSeconds(Cycle);
		ThreadState.LastCycle = Cycle;

		// Decode value
		double FloatValue = 0.0;
		int64 IntValue = 0;
		bool bIsFloat = false;

		EOpType OpType = static_cast<EOpType>(Op);
		switch (OpType)
		{
		case EOpType::Increment:
			IntValue = 1;
			break;
		case EOpType::Decrement:
			IntValue = -1;
			break;
		case EOpType::AddInteger:
			IntValue = TraceQueryLocal::DecodeZigZag(BufferPtr);
			break;
		case EOpType::SetInteger:
			IntValue = TraceQueryLocal::DecodeZigZag(BufferPtr);
			break;
		case EOpType::AddFloat:
			bIsFloat = true;
			memcpy(&FloatValue, BufferPtr, sizeof(double));
			BufferPtr += sizeof(double);
			break;
		case EOpType::SetFloat:
			bIsFloat = true;
			memcpy(&FloatValue, BufferPtr, sizeof(double));
			BufferPtr += sizeof(double);
			break;
		}

		++NumStatsOps;

		if (!PassesTimeFilter(Time))
		{
			continue;
		}

		// Resolve stat name
		const FStatSpec* Spec = StatSpecs.Find(StatId);
		FString StatName;
		FString GroupName;
		if (Spec)
		{
			StatName = Spec->Name;
			GroupName = Spec->Group;
		}
		else
		{
			StatName = FString::Printf(TEXT("Unknown_%u"), StatId);
		}

		// Apply counter name filter (unified: matches both Stats-sourced and Counters-channel events)
		if (!Options.CounterFilter.IsEmpty())
		{
			if (!StatName.Contains(Options.CounterFilter))
			{
				continue;
			}
		}

		static const char* OpNames[] = { "inc", "dec", "add", "set", "add", "set" };
		const char* OpName = (Op < UE_ARRAY_COUNT(OpNames)) ? OpNames[Op] : "unknown";

		// Emit JSONL
		if (bIsFloat)
		{
			FString Line = FString::Printf(
				TEXT("{\"type\":\"stat\",\"time\":%.6f,\"thread\":%u,\"statId\":%u,\"name\":\"%s\",\"group\":\"%s\",\"op\":\"%s\",\"value\":%.9g}"),
				Time, ThreadId, StatId, *JsonEscape(StatName), *JsonEscape(GroupName), ANSI_TO_TCHAR(OpName), FloatValue);
			EmitJsonLine(TCHAR_TO_UTF8(*Line));
		}
		else
		{
			FString Line = FString::Printf(
				TEXT("{\"type\":\"stat\",\"time\":%.6f,\"thread\":%u,\"statId\":%u,\"name\":\"%s\",\"group\":\"%s\",\"op\":\"%s\",\"value\":%lld}"),
				Time, ThreadId, StatId, *JsonEscape(StatName), *JsonEscape(GroupName), ANSI_TO_TCHAR(OpName), IntValue);
			EmitJsonLine(TCHAR_TO_UTF8(*Line));
		}
	}

	if (BufferPtr != BufferEnd)
	{
		UE_LOGF(LogTraceQuery, Warning, "Stats batch buffer mismatch: %lld bytes remaining", (int64)(BufferEnd - BufferPtr));
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Diagnostics
////////////////////////////////////////////////////////////////////////////////////////////////////

void FJsonTraceAnalyzer::OnDiagnosticsSession2(const FOnEventContext& Context)
{
	const FEventData& EventData = Context.EventData;

	FString Platform, AppName, ProjectName, Branch, BuildVersion, CommandLine;
	EventData.GetString("Platform", Platform);
	EventData.GetString("AppName", AppName);
	EventData.GetString("ProjectName", ProjectName);
	EventData.GetString("Branch", Branch);
	EventData.GetString("BuildVersion", BuildVersion);
	EventData.GetString("CommandLine", CommandLine);
	uint32 Changelist = EventData.GetValue<uint32>("Changelist", 0);
	uint8 ConfigurationType = EventData.GetValue<uint8>("ConfigurationType");
	uint8 TargetType = EventData.GetValue<uint8>("TargetType");

	static const char* ConfigNames[] = { "Unknown", "Debug", "DebugGame", "Development", "Shipping", "Test" };
	static const char* TargetNames[] = { "Unknown", "Game", "Server", "Client", "Editor", "Program" };

	const char* ConfigName = (ConfigurationType < UE_ARRAY_COUNT(ConfigNames)) ? ConfigNames[ConfigurationType] : "Unknown";
	const char* TargetName = (TargetType < UE_ARRAY_COUNT(TargetNames)) ? TargetNames[TargetType] : "Unknown";

	FString Line = FString::Printf(
		TEXT("{\"type\":\"session\",\"platform\":\"%s\",\"appName\":\"%s\",\"projectName\":\"%s\",\"branch\":\"%s\",\"buildVersion\":\"%s\",\"changelist\":%u,\"configuration\":\"%s\",\"targetType\":\"%s\"}"),
		*JsonEscape(Platform), *JsonEscape(AppName), *JsonEscape(ProjectName),
		*JsonEscape(Branch), *JsonEscape(BuildVersion), Changelist,
		ANSI_TO_TCHAR(ConfigName), ANSI_TO_TCHAR(TargetName));
	EmitJsonLine(TCHAR_TO_UTF8(*Line));
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Frame
////////////////////////////////////////////////////////////////////////////////////////////////////

void FJsonTraceAnalyzer::OnBeginFrame(const FOnEventContext& Context)
{
	const FEventData& EventData = Context.EventData;
	uint8 FrameType = EventData.GetValue<uint8>("FrameType");
	uint64 Cycle = EventData.GetValue<uint64>("Cycle");
	double Time = Context.EventTime.AsSeconds(Cycle);

	++NumFrames;

	if (Options.bSummaryOnly || !PassesTimeFilter(Time))
	{
		return;
	}

	FrameGroups.FindOrAdd(FrameType).Add(Time);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Helpers
////////////////////////////////////////////////////////////////////////////////////////////////////

bool FJsonTraceAnalyzer::PassesTimeFilter(double Time) const
{
	if (Options.TimeA >= 0.0 && Time < Options.TimeA)
	{
		return false;
	}
	if (Options.TimeB >= 0.0 && Time > Options.TimeB)
	{
		return false;
	}
	return true;
}

void FJsonTraceAnalyzer::EmitJsonLine(const char* JsonLine)
{
	if (!Options.bSummaryOnly)
	{
		puts(JsonLine);
	}
}

FString FJsonTraceAnalyzer::JsonEscape(const FString& Input)
{
	FString Result;
	Result.Reserve(Input.Len() + 8);

	for (int32 Index = 0; Index < Input.Len(); ++Index)
	{
		TCHAR Ch = Input[Index];
		switch (Ch)
		{
		case TEXT('"'):  Result += TEXT("\\\""); break;
		case TEXT('\\'): Result += TEXT("\\\\"); break;
		case TEXT('\n'): Result += TEXT("\\n");  break;
		case TEXT('\r'): Result += TEXT("\\r");  break;
		case TEXT('\t'): Result += TEXT("\\t");  break;
		default:
			if (Ch < 0x20)
			{
				Result += FString::Printf(TEXT("\\u%04x"), (uint32)Ch);
			}
			else
			{
				Result.AppendChar(Ch);
			}
			break;
		}
	}

	return Result;
}

FJsonTraceAnalyzer::FThreadState& FJsonTraceAnalyzer::GetThreadState(uint32 ThreadId)
{
	TSharedPtr<FThreadState>& Slot = ThreadStates.FindOrAdd(ThreadId);
	if (!Slot.IsValid())
	{
		Slot = MakeShared<FThreadState>();
	}
	return *Slot;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Memory (post-analysis)
////////////////////////////////////////////////////////////////////////////////////////////////////

void FJsonTraceAnalyzer::EmitMemoryTags(const TraceServices::IMemoryProvider& MemProvider)
{
	MemProvider.BeginRead();

	const double StartTime = (Options.TimeA >= 0.0) ? Options.TimeA : 0.0;
	const double EndTime   = (Options.TimeB   >= 0.0) ? Options.TimeB   : 1e30;

	// First pass: collect tags that pass the filter and emit their definitions.
	TArray<TraceServices::FMemoryTagInfo> MatchingTags;
	MemProvider.EnumerateTags([&](const TraceServices::FMemoryTagInfo& Tag)
	{
		if (!Options.MemoryTagFilter.IsEmpty())
		{
			if (!Tag.Name.Contains(Options.MemoryTagFilter, ESearchCase::IgnoreCase))
			{
				return;
			}
		}
		MatchingTags.Add(Tag);

		FString Line = FString::Printf(
			TEXT("{\"type\":\"mem_tag\",\"tagId\":%lld,\"parentId\":%lld,\"name\":\"%s\"}"),
			Tag.Id, Tag.ParentId, *JsonEscape(Tag.Name));
		EmitJsonLine(TCHAR_TO_UTF8(*Line));
	});

	// Second pass: emit time-series samples for each matching tag x tracker.
	MemProvider.EnumerateTrackers([&](const TraceServices::FMemoryTrackerInfo& Tracker)
	{
		for (const TraceServices::FMemoryTagInfo& Tag : MatchingTags)
		{
			if (MemProvider.GetTagSampleCount(Tracker.Id, Tag.Id) == 0)
			{
				continue;
			}

			MemProvider.EnumerateTagSamples(Tracker.Id, Tag.Id, StartTime, EndTime,
				/*bIncludeRangeNeighbors=*/false,
				[&](double Time, double /*Duration*/, const TraceServices::FMemoryTagSample& Sample)
				{
					FString Line = FString::Printf(
						TEXT("{\"type\":\"mem_tag_sample\",\"tagId\":%lld,\"name\":\"%s\",\"trackerId\":%d,\"time\":%.6f,\"value_bytes\":%lld}"),
						Tag.Id, *JsonEscape(Tag.Name), (int32)Tracker.Id, Time, (int64)Sample.Value);
					EmitJsonLine(TCHAR_TO_UTF8(*Line));
				});
		}
	});

	MemProvider.EndRead();
}

void FJsonTraceAnalyzer::EmitMemoryTimeline(const TraceServices::IAllocationsProvider& AllocProvider)
{
	AllocProvider.BeginRead();

	const int32 NumPoints = AllocProvider.GetTimelineNumPoints();
	if (NumPoints == 0)
	{
		AllocProvider.EndRead();
		return;
	}

	int32 StartIndex = 0;
	int32 EndIndex   = NumPoints - 1;

	if (Options.TimeA >= 0.0 || Options.TimeB >= 0.0)
	{
		const double FilterStart = (Options.TimeA >= 0.0) ? Options.TimeA : 0.0;
		const double FilterEnd   = (Options.TimeB   >= 0.0) ? Options.TimeB   : 1e30;
		AllocProvider.GetTimelineIndexRange(FilterStart, FilterEnd, StartIndex, EndIndex);
		if (StartIndex < 0)
		{
			StartIndex = 0;
		}
		if (EndIndex >= NumPoints)
		{
			EndIndex = NumPoints - 1;
		}
	}

	if (StartIndex > EndIndex)
	{
		AllocProvider.EndRead();
		return;
	}

	AllocProvider.EnumerateTimeline(
		TraceServices::IAllocationsProvider::ETimelineU64::MaxTotalAllocatedMemory,
		StartIndex, EndIndex,
		[this](double Time, double Duration, uint64 Value)
		{
			// Last bucket has infinite duration (open-ended). JSON does not support
			// inf  --  emit null so consumers can treat it as unknown/unbounded.
			FString DurationStr = FMath::IsFinite(Duration)
				? FString::Printf(TEXT("%.9g"), Duration)
				: TEXT("null");
			FString Line = FString::Printf(
				TEXT("{\"type\":\"mem_timeline\",\"time\":%.6f,\"duration\":%s,\"total_alloc_bytes\":%llu}"),
				Time, *DurationStr, Value);
			EmitJsonLine(TCHAR_TO_UTF8(*Line));
		});

	AllocProvider.EndRead();
}

void FJsonTraceAnalyzer::EmitAllocationTags(const TraceServices::IAllocationsProvider& AllocProvider)
{
	AllocProvider.BeginRead();

	AllocProvider.EnumerateTags([this](const TCHAR* Name, const TCHAR* FullPath, TraceServices::TagIdType TagId, TraceServices::TagIdType ParentTagId)
	{
		// Skip tags with no name
		if (!Name || *Name == TEXT('\0'))
		{
			return;
		}

		FString NameStr(Name);
		FString FullPathStr(FullPath ? FullPath : Name);

		FString Line = FString::Printf(
			TEXT("{\"type\":\"alloc_tag\",\"tagId\":%u,\"parentTagId\":%u,\"name\":\"%s\",\"fullPath\":\"%s\"}"),
			TagId, ParentTagId, *JsonEscape(NameStr), *JsonEscape(FullPathStr));
		EmitJsonLine(TCHAR_TO_UTF8(*Line));
	});

	AllocProvider.EndRead();
}

void FJsonTraceAnalyzer::EmitPackageMemory(
	const TraceServices::IAllocationsProvider& AllocProvider,
	const TraceServices::IMetadataProvider& MetadataProvider,
	const TraceServices::IDefinitionProvider& DefinitionProvider,
	const FPackageMemoryQuery& Query)
{
	// Narrow schema lookup to its own scope -- Schema pointer is stable post-release
	// (schemas are registered once and never removed during analysis).
	uint16 AssetMetadataType;
	const TraceServices::FMetadataSchema* Schema;
	{
		TraceServices::FProviderReadScopeLock MetaSchemaLock(MetadataProvider);
		AssetMetadataType = MetadataProvider.GetRegisteredMetadataType(TEXT("Asset"));
		if (AssetMetadataType == TraceServices::IMetadataProvider::InvalidMetadataType)
		{
			return;
		}
		Schema = MetadataProvider.GetRegisteredMetadataSchema(AssetMetadataType);
		if (!Schema)
		{
			return;
		}
	}

	// Accumulate size and allocation count per (LLM tag, package, class) triple.
	// TagPackageClassSizes[LlmTag][PackageName][ClassName] = total bytes
	// TagTotals[LlmTag] = total bytes across all packages in that tag (for sorting)
	// Allocations without Asset metadata are counted under package "N/A".
	static const FString NAPackage = TEXT("N/A");
	static const FString NATag     = TEXT("N/A");

	TMap<FString, TMap<FString, TMap<FString, uint64>>> TagPackageClassSizes;
	TMap<FString, TMap<FString, TMap<FString, uint32>>> TagPackageClassCounts;
	TMap<FString, TMap<FString, uint64>> TagPackageTotals;
	TMap<FString, uint64> TagTotals;

	AllocProvider.BeginRead();

	TraceServices::IAllocationsProvider::FQueryParams Params{};
	Params.TimeA = Query.TimeA;
	Params.TimeB = (Query.TimeB >= 0.0) ? Query.TimeB : TNumericLimits<double>::Max();
	Params.TimeC = Query.TimeC;
	Params.TimeD = Query.TimeD;
	Params.Rule  = Query.Rule;

	TraceServices::IAllocationsProvider::FQueryHandle QueryHandle = AllocProvider.StartQuery(Params);

	for (;;)
	{
		const TraceServices::IAllocationsProvider::FQueryStatus Status = AllocProvider.PollQuery(QueryHandle);

		if (Status.Status == TraceServices::IAllocationsProvider::EQueryStatus::Available)
		{
			TraceServices::FProviderReadScopeLock MetaBatchLock(MetadataProvider);
			TraceServices::IAllocationsProvider::FQueryResult QueryResult = Status.NextResult();
			if (!QueryResult)
			{
				continue;
			}

			const TraceServices::IAllocationsProvider::FAllocations* Allocations = QueryResult.Get();
			const uint32 NumAllocs = Allocations->Num();

			for (uint32 Index = 0; Index < NumAllocs; ++Index)
			{
				const TraceServices::IAllocationsProvider::FAllocation* Allocation = Allocations->Get(Index);
				if (!Allocation)
				{
					continue;
				}

				// Resolve LLM tag name (full path, e.g. "UI/UI Text").
				TraceServices::TagIdType LlmTagId = Allocation->GetTag();
				const TCHAR* TagPathRaw = AllocProvider.GetTagFullPath(LlmTagId);
				FString LlmTagName = (TagPathRaw && *TagPathRaw) ? FString(TagPathRaw) : NATag;

				const uint64 AllocSize = Allocation->GetSize();
				const uint32 MetadataId = Allocation->GetMetadataId();

				if (MetadataId == TraceServices::IMetadataProvider::InvalidMetadataId)
				{
					// No Asset metadata  --  count under "N/A" package within the LLM tag.
					TagPackageClassSizes.FindOrAdd(LlmTagName).FindOrAdd(NAPackage).FindOrAdd(TEXT("Unknown")) += AllocSize;
					TagPackageClassCounts.FindOrAdd(LlmTagName).FindOrAdd(NAPackage).FindOrAdd(TEXT("Unknown")) += 1;
					TagPackageTotals.FindOrAdd(LlmTagName).FindOrAdd(NAPackage) += AllocSize;
					TagTotals.FindOrAdd(LlmTagName) += AllocSize;
					continue;
				}

				MetadataProvider.EnumerateMetadata(
					Allocation->GetAllocThreadId(), MetadataId,
					[AssetMetadataType, Schema, &Allocation, &LlmTagName, &AllocSize,
					 &TagPackageClassSizes, &TagPackageClassCounts, &TagPackageTotals, &TagTotals,
					 &DefinitionProvider]
					(uint32 StackDepth, uint16 Type, const void* Data, uint32 Size) -> bool
					{
						if (Type == AssetMetadataType)
						{
							TraceServices::FProviderReadScopeLock DefReadLock(DefinitionProvider);
							const auto Reader = Schema->Reader();

							// Field 2 = Package, Field 1 = Class (from AssetMetadataTrace.h schema)
							const auto* PackageNameRef = Reader.GetValueAs<UE::Trace::FEventRef32>((uint8*)Data, 2);
							const auto* ClassNameRef   = Reader.GetValueAs<UE::Trace::FEventRef32>((uint8*)Data, 1);

							if (!PackageNameRef)
							{
								return true; // malformed schema -- skip this allocation
							}
							const auto* PackageDef = DefinitionProvider.Get<TraceServices::FStringDefinition>(*PackageNameRef);
							FString PackageName = (PackageDef && PackageDef->Display && *PackageDef->Display != TEXT('\0'))
								? FString(PackageDef->Display) : TEXT("N/A");

							FString ClassName;
							if (ClassNameRef)
							{
								const auto* ClassDef = DefinitionProvider.Get<TraceServices::FStringDefinition>(*ClassNameRef);
								if (ClassDef && ClassDef->Display && *ClassDef->Display != TEXT('\0'))
								{
									ClassName = FString(ClassDef->Display);
								}
							}
							if (ClassName.IsEmpty())
							{
								ClassName = TEXT("Unknown");
							}

							TagPackageClassSizes.FindOrAdd(LlmTagName).FindOrAdd(PackageName).FindOrAdd(ClassName) += AllocSize;
							TagPackageClassCounts.FindOrAdd(LlmTagName).FindOrAdd(PackageName).FindOrAdd(ClassName) += 1;
							TagPackageTotals.FindOrAdd(LlmTagName).FindOrAdd(PackageName) += AllocSize;
							TagTotals.FindOrAdd(LlmTagName) += AllocSize;

							return false; // found Asset metadata, stop enumerating stack
						}
						return true; // keep looking up the stack
					});
			}
		}
		else if (Status.Status == TraceServices::IAllocationsProvider::EQueryStatus::Working)
		{
			FPlatformProcess::Sleep(0.01f);
		}
		else // Done or Unknown
		{
			break;
		}
	}

	AllocProvider.CancelQuery(QueryHandle);
	AllocProvider.EndRead();

	// Sort LLM tags by total size descending, packages within each tag by size desc.
	TArray<FString> TagNames;
	TagTotals.GetKeys(TagNames);
	TagNames.Sort([&TagTotals](const FString& A, const FString& B)
	{
		return TagTotals[A] > TagTotals[B];
	});

	// Emit one JSONL record per (LLM tag, package) with an inline classes array.
	// Field "llm_tag" carries the tag full path so consumers can reconstruct the hierarchy.
	for (const FString& TagName : TagNames)
	{
		const TMap<FString, uint64>& PkgTotals = TagPackageTotals[TagName];

		TArray<FString> PackageNames;
		PkgTotals.GetKeys(PackageNames);
		PackageNames.Sort([&PkgTotals](const FString& A, const FString& B)
		{
			return PkgTotals[A] > PkgTotals[B];
		});

		for (const FString& PkgName : PackageNames)
		{
			uint64 PkgTotalBytes = PkgTotals[PkgName];

			const TMap<FString, uint64>& ClassSizes  = TagPackageClassSizes[TagName][PkgName];
			const TMap<FString, uint32>& ClassCounts = TagPackageClassCounts[TagName][PkgName];

			TArray<FString> ClassNames;
			ClassSizes.GetKeys(ClassNames);
			ClassNames.Sort([&ClassSizes](const FString& A, const FString& B)
			{
				return ClassSizes[A] > ClassSizes[B];
			});

			uint32 PkgTotalCount = 0;
			for (const FString& CN : ClassNames)
			{
				PkgTotalCount += ClassCounts.FindRef(CN);
			}

			FString ClassesArray = TEXT("[");
			for (int32 CI = 0; CI < ClassNames.Num(); ++CI)
			{
				if (CI > 0)
			{
				ClassesArray += TEXT(",");
			}
				const FString& CN = ClassNames[CI];
				ClassesArray += FString::Printf(
					TEXT("{\"name\":\"%s\",\"size_bytes\":%llu,\"alloc_count\":%u}"),
					*JsonEscape(CN), ClassSizes[CN], ClassCounts.FindRef(CN));
			}
			ClassesArray += TEXT("]");

			FString Line;
			if (Query.Id.IsEmpty())
			{
				Line = FString::Printf(
					TEXT("{\"type\":\"pkg_mem\",\"llm_tag\":\"%s\",\"name\":\"%s\",\"size_bytes\":%llu,\"alloc_count\":%u,\"classes\":%s}"),
					*JsonEscape(TagName), *JsonEscape(PkgName), PkgTotalBytes, PkgTotalCount, *ClassesArray);
			}
			else
			{
				Line = FString::Printf(
					TEXT("{\"type\":\"pkg_mem\",\"query_id\":\"%s\",\"llm_tag\":\"%s\",\"name\":\"%s\",\"size_bytes\":%llu,\"alloc_count\":%u,\"classes\":%s}"),
					*JsonEscape(Query.Id), *JsonEscape(TagName), *JsonEscape(PkgName),
					PkgTotalBytes, PkgTotalCount, *ClassesArray);
			}
			EmitJsonLine(TCHAR_TO_UTF8(*Line));
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FJsonTraceAnalyzer::EmitLogMessages(const TraceServices::ILogProvider& LogProvider)
{
	// Enumerate log messages within the requested time range (or full trace if no filter).
	// Each record: {"type":"log_msg","time_s":N.NNN,"category":"LogFort","verbosity":"Warning","message":"..."}
	const double StartS = (Options.TimeA >= 0.0) ? Options.TimeA : 0.0;
	const double EndS   = (Options.TimeB   >= 0.0) ? Options.TimeB   : TNumericLimits<double>::Max();

	TraceServices::FProviderReadScopeLock LogReadLock(LogProvider);
	LogProvider.EnumerateMessages(StartS, EndS, [&](const TraceServices::FLogMessageInfo& Msg)
	{
		if (!Msg.Message || !*Msg.Message)
		{
			return;
		}

		const TCHAR* CategoryName = (Msg.Category && Msg.Category->Name) ? Msg.Category->Name : TEXT("");
		const TCHAR* VerbosityStr = ::ToString(Msg.Verbosity);

		FString Line = FString::Printf(
			TEXT("{\"type\":\"log_msg\",\"time_s\":%.6f,\"category\":\"%s\",\"verbosity\":\"%s\",\"message\":\"%s\"}"),
			Msg.Time,
			*JsonEscape(FString(CategoryName)),
			*JsonEscape(FString(VerbosityStr)),
			*JsonEscape(FString(Msg.Message)));
		EmitJsonLine(TCHAR_TO_UTF8(*Line));
	});
}

void FJsonTraceAnalyzer::EmitRegions(const TraceServices::IRegionProvider& RegionProvider)
{
	// Enumerate all timing regions from the default timeline.
	// Each record: {"type":"region","name":"...","begin_s":N.NNN,"end_s":N.NNN}
	// end_s is -1.0 for open-ended regions (TRACE_BEGIN_REGION with no matching TRACE_END_REGION).
	RegionProvider.BeginRead();

	const TraceServices::IRegionTimeline& Timeline = RegionProvider.GetDefaultTimeline();
	Timeline.EnumerateLanes([&](const TraceServices::FRegionLane& Lane, const int32 /*Depth*/)
	{
		Lane.EnumerateRegions(0.0, TNumericLimits<double>::Max(),
			[&](const TraceServices::FTimeRegion& Region) -> bool
		{
			const TCHAR* Name = Region.Timer ? Region.Timer->Name : nullptr;
			if (!Name || !*Name)
			{
				return true;
			}

			const double BeginS = Region.BeginTime;
			const bool bOpen    = !FMath::IsFinite(Region.EndTime) || Region.EndTime >= 1e29;
			const double EndS   = bOpen ? -1.0 : Region.EndTime;

			if (!PassesTimeFilter(BeginS))
			{
				return true;
			}

			FString Line = FString::Printf(
				TEXT("{\"type\":\"region\",\"name\":\"%s\",\"begin_s\":%.6f,\"end_s\":%.6f}"),
				*JsonEscape(FString(Name)), BeginS, EndS);
			EmitJsonLine(TCHAR_TO_UTF8(*Line));
			return true;
		});
	});

	RegionProvider.EndRead();
}
