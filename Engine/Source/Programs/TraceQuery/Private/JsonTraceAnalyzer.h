// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraceQuery.h"

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "Trace/Analyzer.h"
#include "Common/ProviderLock.h"
#include "TraceServices/Model/AllocationsProvider.h"
#include "TraceServices/Model/Counters.h"
#include "TraceServices/Model/Definitions.h"
#include "TraceServices/Model/Memory.h"
#include "TraceServices/Model/MetadataProvider.h"
#include "TraceServices/Model/Log.h"
#include "TraceServices/Model/Regions.h"
#include "TraceServices/Model/Strings.h"
#include "TraceServices/Model/Threads.h"
#include "TraceServices/Model/TimingProfiler.h"

////////////////////////////////////////////////////////////////////////////////////////////////////
// Shared data types used by providers and JsonTraceAnalyzer
////////////////////////////////////////////////////////////////////////////////////////////////////

// Key for grouping CPU scope events by (ThreadId, SpecId)
struct FScopeGroupKey
{
	uint32 ThreadId;
	uint32 SpecId;

	bool operator==(const FScopeGroupKey& Other) const
	{
		return ThreadId == Other.ThreadId && SpecId == Other.SpecId;
	}
};

inline uint32 GetTypeHash(const FScopeGroupKey& Key)
{
	return HashCombine(GetTypeHash(Key.ThreadId), GetTypeHash(Key.SpecId));
}

struct FScopeDataPoint
{
	double Time = 0.0;
	double Duration = 0.0;
	int32 Depth = 0;
	uint32 ParentSpecId = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTraceQueryCpuProfilerProvider
//
// Lightweight IEditableTimingProfilerProvider + IEditableThreadProvider that receives decoded
// CPU profiler events from TraceServices' FCpuProfilerAnalyzer and populates ScopeGroups.
// Modeled after FSummarizeCpuProfilerProvider in SummarizeTraceUtils.h.
////////////////////////////////////////////////////////////////////////////////////////////////////

class FTraceQueryCpuProfilerProvider
	: public TraceServices::IEditableThreadProvider
	, public TraceServices::IEditableTimingProfilerProvider
{
public:
	// Wired by TraceQuery.cpp before analysis begins
	const FTraceQueryOptions* Options = nullptr;
	TMap<FScopeGroupKey, TArray<FScopeDataPoint>>* ScopeGroups = nullptr;
	uint64* NumCpuSpecs = nullptr;
	uint64* NumCpuEvents = nullptr;

	// Populated when Options->bWantCsvFrameNumbers is set.
	// Each entry is {frame_start_time_s, csv_frame_number}.
	TArray<TPair<double, int64>>* CsvFrameNumbers = nullptr;

	// Resolve a TimerIndex (which may be a ~MetadataId) to its scope name, or nullptr
	const FString* LookupTimerName(uint32 TimerId) const;

	// Resolve a ThreadId to its human-readable name (e.g. "GameThread"), or nullptr
	const FString* LookupThreadName(uint32 ThreadId) const;

	// Store a thread name (called by FJsonTraceAnalyzer::OnThreadInfo)
	void SetThreadName(uint32 ThreadId, const FString& Name);

private:
	struct FScopeEnter
	{
		uint32 TimerId;
		double StartTime;
	};

	struct FThread : public TraceServices::IEditableTimeline<TraceServices::FTimingProfilerEvent>
	{
		FThread(uint32 InThreadId, FTraceQueryCpuProfilerProvider* InProvider)
			: ThreadId(InThreadId)
			, Provider(InProvider)
		{
		}

		virtual void AppendBeginEvent(double StartTime, const TraceServices::FTimingProfilerEvent& Event) override;
		virtual void AppendEndEvent(double EndTime) override;

		uint32 ThreadId;
		FTraceQueryCpuProfilerProvider* Provider;
		TArray<FScopeEnter> ScopeStack;
	};

	struct FMetadata
	{
		TArray<uint8> Payload;
		uint32 TimerId;
	};

	// IEditableThreadProvider
	virtual void AddThread(uint32 Id, const TCHAR* Name, EThreadPriority Priority) override;

	// IEditableTimingProfilerProvider -- methods called by FCpuProfilerAnalyzer
	virtual uint32 AddTimer(TraceServices::ETimingProfilerTimerType Type) override;
	virtual uint32 AddCpuTimer(FStringView Name, const TCHAR* File, uint32 Line) override;
	virtual void SetTimerName(uint32 TimerId, FStringView Name) override;
	virtual uint32 AddMetadata(uint32 MainTimerId, TArray<uint8>&& Metadata) override;
	virtual TArrayView<uint8> GetEditableMetadata(uint32 TimerId) override;
	virtual TraceServices::IEditableTimeline<TraceServices::FTimingProfilerEvent>& GetCpuThreadEditableTimeline(uint32 ThreadId) override;

	uint32 AddTimerInternal(FStringView Name);

	TArray<TOptional<FString>> TimerNames;
	TMap<uint32, FString> ThreadNames;
	TMap<uint32, TUniquePtr<FThread>> Threads;
	TArray<FMetadata> Metadatas;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTraceQueryCounter + FTraceQueryCounterProvider
//
// Lightweight IEditableCounter / IEditableCounterProvider that streams counter values
// as JSONL via a callback. Modeled after FSummarizeCounter in SummarizeTraceCommandlet.cpp.
////////////////////////////////////////////////////////////////////////////////////////////////////

class FTraceQueryCounter : public TraceServices::IEditableCounter
{
public:
	// Wired by provider on creation
	const FTraceQueryOptions* Options = nullptr;
	TFunction<void(const char*)> EmitCallback;
	TFunction<bool(double)> TimeFilterCallback;
	TFunction<FString(const FString&)> JsonEscapeCallback;
	uint64* NumCounterValues = nullptr;

	uint16 CounterId = 0;

	// IEditableCounter
	virtual void SetName(const TCHAR* InName) override { Name = InName; }
	virtual void SetGroup(const TCHAR* Group) override {}
	virtual void SetDescription(const TCHAR* Description) override {}
	virtual void SetIsFloatingPoint(bool bIsFloatingPoint) override { bIsFloat = bIsFloatingPoint; }
	virtual void SetIsResetEveryFrame(bool bInIsResetEveryFrame) override {}
	virtual void SetDisplayHint(TraceServices::ECounterDisplayHint DisplayHint) override {}

	virtual void AddValue(double Time, int64 Value) override { SetValue(Time, Value); }
	virtual void SetValue(double Time, int64 Value) override;
	virtual void AddValue(double Time, double Value) override { SetValue(Time, Value); }
	virtual void SetValue(double Time, double Value) override;

private:
	FString Name;
	bool bIsFloat = false;
};

class FTraceQueryCounterProvider : public TraceServices::IEditableCounterProvider
{
public:
	// Wired by TraceQuery.cpp before analysis begins
	const FTraceQueryOptions* Options = nullptr;
	TFunction<void(const char*)> EmitCallback;
	TFunction<bool(double)> TimeFilterCallback;
	TFunction<FString(const FString&)> JsonEscapeCallback;
	uint64* NumCounterSpecs = nullptr;
	uint64* NumCounterValues = nullptr;

	// IEditableCounterProvider
	virtual const TraceServices::ICounter* GetCounter(TraceServices::IEditableCounter* EditableCounter) override { return nullptr; }
	virtual TraceServices::IEditableCounter* CreateEditableCounter() override;
	virtual void AddCounter(const TraceServices::ICounter* Counter) override {}

private:
	uint16 NextCounterId = 0;
	TArray<TUniquePtr<FTraceQueryCounter>> Counters;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// Forward declaration so FJsonTraceAnalyzer can reference FPackageMemoryQuery in its declaration.
// Full definition follows the class closing brace.
struct FPackageMemoryQuery;

////////////////////////////////////////////////////////////////////////////////////////////////////
// FJsonTraceAnalyzer
//
// IAnalyzer that handles Stats, Diagnostics, BeginFrame, and AllEvents.
// CpuProfiler and Counters are handled by TraceServices factory analyzers with custom providers.
////////////////////////////////////////////////////////////////////////////////////////////////////

class FJsonTraceAnalyzer : public UE::Trace::IAnalyzer
{
public:
	explicit FJsonTraceAnalyzer(const FTraceQueryOptions& InOptions);

	// Escape a string for JSON output (handles quotes, backslashes, control chars)
	static FString JsonEscape(const FString& Input);

	// Output helpers -- public so providers can access them
	bool PassesTimeFilter(double Time) const;
	void EmitJsonLine(const char* JsonLine);

	// Post-analysis emission -- called from TraceQuery.cpp after Process().Wait()
	void EmitMemoryTags(const TraceServices::IMemoryProvider& MemProvider);
	void EmitMemoryTimeline(const TraceServices::IAllocationsProvider& AllocProvider);
	void EmitAllocationTags(const TraceServices::IAllocationsProvider& AllocProvider);
	void EmitPackageMemory(
		const TraceServices::IAllocationsProvider& AllocProvider,
		const TraceServices::IMetadataProvider& MetadataProvider,
		const TraceServices::IDefinitionProvider& DefinitionProvider,
		const FPackageMemoryQuery& Query);
	void EmitRegions(const TraceServices::IRegionProvider& RegionProvider);
	void EmitLogMessages(const TraceServices::ILogProvider& LogProvider);

	// Buffered CPU scope data -- populated by CpuProfiler provider, flushed in OnAnalysisEnd
	TMap<FScopeGroupKey, TArray<FScopeDataPoint>> ScopeGroups;

	// CSV frame numbers -- populated by CpuProfiler provider when bWantCsvFrameNumbers is set.
	// Entries: {frame_start_time_s, csv_frame_number}, emitted as csv_frame_number records.
	TArray<TPair<double, int64>> CsvFrameNumbers;

	// Summary mode counters -- providers write to these via pointers
	uint64 NumStatsSpecs = 0;
	uint64 NumStatsBatches = 0;
	uint64 NumStatsOps = 0;
	uint64 NumCounterSpecs = 0;
	uint64 NumCounterValues = 0;
	uint64 NumFrames = 0;
	uint64 NumCpuSpecs = 0;
	uint64 NumCpuEvents = 0;
	uint64 NumTotalEvents = 0;

	// Set by TraceQuery.cpp to enable timer/thread name resolution in OnAnalysisEnd
	FTraceQueryCpuProfilerProvider* CpuProfilerProvider = nullptr;

private:
	// IAnalyzer interface
	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual void OnAnalysisEnd() override;
	virtual void OnThreadInfo(const FThreadInfo& ThreadInfo) override;
	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;

	// Stats decode (kept -- no factory exists for Stats)
	void OnStatsSpec(const FOnEventContext& Context);
	void OnStatsEventBatch(const FOnEventContext& Context, bool bV2);

	// Simple event handlers
	void OnDiagnosticsSession2(const FOnEventContext& Context);
	void OnBeginFrame(const FOnEventContext& Context);

	enum : uint16
	{
		RouteId_StatsSpec,
		RouteId_StatsEventBatch,
		RouteId_StatsEventBatch2,
		RouteId_DiagSession,
		RouteId_DiagSession2,
		RouteId_BeginFrame,
		RouteId_AllEvents,
	};

	struct FStatSpec
	{
		FString Name;
		FString Group;
		FString Description;
		bool bIsFloatingPoint = false;
		bool bIsResetEveryFrame = false;
	};

	// Per-thread state for Stats batch decode
	struct FThreadState
	{
		uint64 LastCycle = 0;
	};

	const FTraceQueryOptions& Options;

	TMap<uint32, FStatSpec> StatSpecs;
	TMap<uint32, TSharedPtr<FThreadState>> ThreadStates;

	// Buffered frame timestamps -- flushed as grouped JSONL in OnAnalysisEnd
	// Key: frameType (0=game, 1=render)
	TMap<uint8, TArray<double>> FrameGroups;

	FThreadState& GetThreadState(uint32 ThreadId);
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// FPackageMemoryQuery -- parameters for a single EmitPackageMemory call.
// Id is echoed as "query_id" in output records; empty = single-query (no field emitted).

struct FPackageMemoryQuery
{
	FString Id;
	TraceServices::IAllocationsProvider::EQueryRule Rule =
		TraceServices::IAllocationsProvider::EQueryRule::Aaf;
	double TimeA = 0.0;
	double TimeB = -1.0;
	double TimeC = 0.0;
	double TimeD = 0.0;
};
