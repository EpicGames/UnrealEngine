// Copyright Epic Games, Inc. All Rights Reserved.

#include "RegionAnalyzer.h"

#include "Common/ProviderLock.h"
#include "Containers/UnrealString.h"

void FRegionAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	auto& Builder = Context.InterfaceBuilder;
	Builder.RouteEvent(RouteId_RegionBegin,       "Misc", "RegionBegin");
	Builder.RouteEvent(RouteId_RegionBeginWithId, "Misc", "RegionBeginWithId");
	Builder.RouteEvent(RouteId_RegionEnd,         "Misc", "RegionEnd");
	Builder.RouteEvent(RouteId_RegionEndWithId,   "Misc", "RegionEndWithId");
}

void FRegionAnalyzer::OnAnalysisEnd()
{
	TraceServices::FProviderEditScopeLock Lock(RegionProvider);
	RegionProvider.OnAnalysisSessionEnded();
}

bool FRegionAnalyzer::OnEvent(uint16 RouteId, EStyle /*Style*/, const FOnEventContext& Context)
{
	const auto& EventData = Context.EventData;
	switch (RouteId)
	{
	case RouteId_RegionBegin:
	{
		const uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		FString Name; check(EventData.GetString("RegionName", Name));
		FString Category; EventData.GetString("Category", Category);
		DispatchEvent(Context, Cycle, [&](double Time)
		{
			RegionProvider.AppendRegionBegin(*Name, Time, Category.IsEmpty() ? nullptr : *Category);
		});
		break;
	}
	case RouteId_RegionBeginWithId:
	{
		const uint64 CycleAndId = EventData.GetValue<uint64>("CycleAndId");
		FString Name; check(EventData.GetString("RegionName", Name));
		FString Category; EventData.GetString("Category", Category);
		DispatchEvent(Context, CycleAndId, [&](double Time)
		{
			RegionProvider.AppendRegionBeginWithId(*Name, CycleAndId, Time, Category.IsEmpty() ? nullptr : *Category);
		});
		break;
	}
	case RouteId_RegionEnd:
	{
		const uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		FString Name; EventData.GetString("RegionName", Name);
		DispatchEvent(Context, Cycle, [&](double Time)
		{
			RegionProvider.AppendRegionEnd(*Name, Time);
		});
		break;
	}
	case RouteId_RegionEndWithId:
	{
		const uint64 Cycle    = EventData.GetValue<uint64>("Cycle");
		const uint64 RegionId = EventData.GetValue<uint64>("RegionId", 0);
		DispatchEvent(Context, Cycle, [&](double Time)
		{
			RegionProvider.AppendRegionEndWithId(RegionId, Time);
		});
		break;
	}
	}
	return true;
}
