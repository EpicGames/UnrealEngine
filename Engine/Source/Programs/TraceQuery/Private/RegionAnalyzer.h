// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Analyzer.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "Model/RegionsPrivate.h"

////////////////////////////////////////////////////////////////////////////////////////////////////
// FRegionAnalyzer -- routes the 4 Misc region events into FRegionProvider.
// Defined here rather than in TraceServices because it only exists to serve TraceQuery;
// the standard FMiscTraceAnalyzer would require FThreadProvider, FLogProvider,
// FFrameProvider, FChannelProvider, and FScreenshotProvider that TraceQuery doesn't need.
////////////////////////////////////////////////////////////////////////////////////////////////////

struct FRegionAnalyzer : public UE::Trace::IAnalyzer
{
	FRegionAnalyzer(TraceServices::IAnalysisSession& InSession,
	                TraceServices::FRegionProvider& InProvider)
		: Session(InSession), RegionProvider(InProvider) {}

	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual void OnAnalysisEnd() override;
	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;

private:
	// Factors out the repeated lock + UpdateDurationSeconds pattern across all 4 OnEvent cases.
	template <typename TCallback>
	void DispatchEvent(const FOnEventContext& Context, uint64 CycleValue, TCallback&& ProviderCall)
	{
		const double Time = Context.EventTime.AsSeconds(CycleValue);
		{ TraceServices::FProviderEditScopeLock Lock(RegionProvider);
		  ProviderCall(Time); }
		Session.UpdateDurationSeconds(Time);
	}

	enum : uint16 { RouteId_RegionBegin, RouteId_RegionBeginWithId,
	                RouteId_RegionEnd,   RouteId_RegionEndWithId };
	TraceServices::IAnalysisSession&        Session;
	TraceServices::IEditableRegionProvider& RegionProvider;
};
