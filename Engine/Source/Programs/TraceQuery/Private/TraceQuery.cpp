// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceQuery.h"
#include "JsonTraceAnalyzer.h"

#include "Misc/CommandLine.h"
#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "Misc/FileHelper.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "RequiredProgramMainCPPInclude.h"

// TraceAnalysis
#include "Trace/Analysis.h"
#include "Trace/DataStream.h"

// TraceServices  --  factory analyzers for CpuProfiler, Counters, Memory, Regions, and Log
#include "TraceServices/AnalyzerFactories.h"
#include "TraceServices/Model/AllocationsProvider.h"
#include "TraceServices/Model/Definitions.h"
#include "TraceServices/Model/Log.h"
#include "TraceServices/Model/Memory.h"
#include "TraceServices/Model/MetadataProvider.h"
#include "TraceServices/Model/Regions.h"
#include "TraceServices/Model/Strings.h"

// Private TraceServices headers for concrete provider and analyzer types.
// Access granted via PrivateIncludePaths in TraceQuery.Build.cs.
#include "Model/AllocationsProvider.h"    // FAllocationsProvider
#include "Model/DefinitionProvider.h"     // FDefinitionProvider
#include "Model/LogPrivate.h"             // FLogProvider
#include "Model/MemoryPrivate.h"          // FMemoryProvider
#include "Model/MetadataProvider.h"       // FMetadataProvider
#include "Analyzers/AllocationsAnalysis.h" // FAllocationsAnalyzer
#include "Analyzers/LogTraceAnalysis.h"    // FLogTraceAnalyzer
#include "Analyzers/MemoryAnalysis.h"      // FMemoryAnalyzer
#include "Analyzers/MetadataAnalysis.h"    // FMetadataAnalysis
#include "Analyzers/StringsAnalyzer.h"     // FStringsAnalyzer

#include <stdio.h>

DEFINE_LOG_CATEGORY(LogTraceQuery);

////////////////////////////////////////////////////////////////////////////////////////////////////
// Local analyzer creation -- avoids the TraceServices factory functions which require
// interface downcasts. Uses concrete F...Provider types directly. 
// Insights team doesn't want these publicly exposed, which would require altering constructors to take interface classes

namespace
{
	TSharedPtr<UE::Trace::IAnalyzer> MakeMemoryAnalyzer(TraceServices::IAnalysisSession& Session, TraceServices::FMemoryProvider& Provider)
	{
		return MakeShared<TraceServices::FMemoryAnalyzer>(Session, &Provider);
	}

	TSharedPtr<UE::Trace::IAnalyzer> MakeMetadataAnalyzer(TraceServices::IAnalysisSession& Session, TraceServices::FMetadataProvider& Provider)
	{
		return MakeShared<TraceServices::FMetadataAnalysis>(Session, &Provider);
	}

	TSharedPtr<UE::Trace::IAnalyzer> MakeAllocationsAnalyzer(TraceServices::IAnalysisSession& Session, TraceServices::FAllocationsProvider& AllocsProvider, TraceServices::FMetadataProvider& MetaProvider)
	{
		return MakeShared<TraceServices::FAllocationsAnalyzer>(Session, AllocsProvider, MetaProvider);
	}

	TSharedPtr<UE::Trace::IAnalyzer> MakeLogAnalyzer(TraceServices::IAnalysisSession& Session, TraceServices::FLogProvider& Provider)
	{
		return MakeShared<TraceServices::FLogTraceAnalyzer>(Session, Provider);
	}
}

IMPLEMENT_APPLICATION(TraceQuery, "TraceQuery");

#include "RegionAnalyzer.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

using EQueryRule = TraceServices::IAllocationsProvider::EQueryRule;

static const EQueryRule* FindQueryRule(const FString& RuleName)
{
	static const TMap<FString, EQueryRule> RuleMap = {
		{ TEXT("aAf"),   EQueryRule::aAf   }, { TEXT("afA"),   EQueryRule::afA   },
		{ TEXT("Aaf"),   EQueryRule::Aaf   }, { TEXT("aAfB"),  EQueryRule::aAfB  },
		{ TEXT("AaBf"),  EQueryRule::AaBf  }, { TEXT("AfB"),   EQueryRule::AfB   },
		{ TEXT("AaB"),   EQueryRule::AaB   }, { TEXT("AafB"),  EQueryRule::AafB  },
		{ TEXT("aABf"),  EQueryRule::aABf  },
		{ TEXT("AaBCf"), EQueryRule::AaBCf }, { TEXT("AaBfC"), EQueryRule::AaBfC },
		{ TEXT("aABfC"), EQueryRule::aABfC }, { TEXT("AaBCfD"),EQueryRule::AaBCfD},
	};
	return RuleMap.Find(RuleName);
}

static bool ValidateQueryFileRules(const FString& FilePath)
{
	FString FileContent;
	if (!FFileHelper::LoadFileToString(FileContent, *FilePath))
	{
		fprintf(stderr, "Error: Cannot read query file '%s'\n", TCHAR_TO_ANSI(*FilePath));
		return false;
	}
	TSharedPtr<FJsonValue> Root;
	TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(FileContent);
	if (!FJsonSerializer::Deserialize(Reader, Root) || !Root.IsValid())
	{
		fprintf(stderr, "Error: Failed to parse query file '%s'\n", TCHAR_TO_ANSI(*FilePath));
		return false;
	}
	const TArray<TSharedPtr<FJsonValue>>* Array;
	if (!Root->TryGetArray(Array))
	{
		fprintf(stderr, "Error: Query file must be a JSON array\n");
		return false;
	}
	int32 EntryIdx = 0;
	for (const TSharedPtr<FJsonValue>& Entry : *Array)
	{
		const TSharedPtr<FJsonObject>* ObjPtr;
		if (!Entry->TryGetObject(ObjPtr)) { ++EntryIdx; continue; }
		const TSharedPtr<FJsonObject>& Obj = *ObjPtr;

		FString IdStr;
		if (!Obj->TryGetStringField(TEXT("id"), IdStr) || IdStr.IsEmpty())
		{
			fprintf(stderr, "Error: Query file entry %d missing required 'id' field\n", EntryIdx);
			return false;
		}

		FString RuleStr;
		if (!Obj->TryGetStringField(TEXT("rule"), RuleStr))
		{
			fprintf(stderr, "Error: Query '%s' (entry %d) missing required 'rule' field\n",
				TCHAR_TO_ANSI(*IdStr), EntryIdx);
			return false;
		}
		if (!FindQueryRule(RuleStr))
		{
			fprintf(stderr, "Error: Unknown rule '%s' in query '%s'. Valid rules: aAf, afA, Aaf, aAfB, AaBf, AfB, AaB, AafB, aABf, AaBCf, AaBfC, aABfC, AaBCfD\n",
				TCHAR_TO_ANSI(*RuleStr), TCHAR_TO_ANSI(*IdStr));
			return false;
		}
		++EntryIdx;
	}
	return true;
}

static void PrintUsage()
{
	fprintf(stderr, "Usage:\n");
	fprintf(stderr, "  TraceQuery <file.utrace> [options]\n");
	fprintf(stderr, "\n");
	fprintf(stderr, "Options:\n");
	fprintf(stderr, "  -logger=<name>        Filter by logger (Stats, Counters, CpuProfiler, MemoryTags,\n");
	fprintf(stderr, "                        MemoryTimeline, MemoryAssets, MemoryPackages, Memory,\n");
	fprintf(stderr, "                        Regions, Log, CsvFrames, or *)\n");
	fprintf(stderr, "  -event=<pattern>      Filter by event name (* wildcard)\n");
	fprintf(stderr, "  -counter=<name>       Filter counters by name substring (both Stats and Counters channels)\n");
	fprintf(stderr, "  -timers=<name>        Filter CPU profiler scopes by name substring\n");
	fprintf(stderr, "  -memtag=<filter>      Only emit LLM tags whose name contains <filter> (case-insensitive)\n");
	fprintf(stderr, "  -summary              Print event type summary, exit\n");
	fprintf(stderr, "  -timeA=<sec>          First time marker (alias: -startTime). For most loggers: only emit events after this time.\n");
	fprintf(stderr, "  -timeB=<sec>          Second time marker (alias: -endTime). For most loggers: only emit events before this time.\n");
	fprintf(stderr, "  -timeC=<sec>          Third time marker (MemoryPackages only, for 3-marker rules)\n");
	fprintf(stderr, "  -timeD=<sec>          Fourth time marker (MemoryPackages only, for 4-marker rule AaBCfD)\n");
	fprintf(stderr, "  -startTime=<sec>      Alias for -timeA\n");
	fprintf(stderr, "  -endTime=<sec>        Alias for -timeB\n");
	fprintf(stderr, "  -rule=<rule>          Explicit query rule for MemoryPackages (overrides auto-selection from -timeA/-timeB)\n");
	fprintf(stderr, "  -queryFile=<path>     JSON file with multiple MemoryPackages queries\n");
	fprintf(stderr, "                        Array of objects: [{\"id\":\"<str>\",\"rule\":\"<rule>\",\n");
	fprintf(stderr, "                          \"timeA\":<sec>,\"timeB\":<sec>,\"timeC\":<sec>,\"timeD\":<sec>},...]\n");
	fprintf(stderr, "                        Required fields: id, rule. Optional: timeA/startTime, timeB/endTime, timeC, timeD\n");
	fprintf(stderr, "                        Valid rules (1-marker): aAf, afA, Aaf\n");
	fprintf(stderr, "                        Valid rules (2-marker): aAfB, AaBf, AfB, AaB, AafB, aABf\n");
	fprintf(stderr, "                        Valid rules (3-marker): AaBCf, AaBfC, aABfC  (use timeC)\n");
	fprintf(stderr, "                        Valid rules (4-marker): AaBCfD  (use timeC and timeD)\n");
	fprintf(stderr, "                        Runs all queries in one pass; ignores -timeA/-timeB\n");
	fprintf(stderr, "\n");
	fprintf(stderr, "Memory loggers (for .utrace memory captures):\n");
	fprintf(stderr, "  -logger=MemoryTags      Emit mem_tag and mem_tag_sample records (LLM tag values)\n");
	fprintf(stderr, "  -logger=MemoryTimeline  Emit mem_timeline records (total allocated memory over time)\n");
	fprintf(stderr, "  -logger=MemoryAssets    Emit alloc_tag records (per-asset allocation tag names)\n");
	fprintf(stderr, "  -logger=MemoryPackages  Emit package_memory records (per-package sizes, LLM Tag/Package/Class hierarchy)\n");
	fprintf(stderr, "  -logger=Memory          MemoryTags + MemoryTimeline + MemoryAssets + MemoryPackages\n");
	fprintf(stderr, "\n");
	fprintf(stderr, "Other loggers:\n");
	fprintf(stderr, "  -logger=Log         Emit log_msg records (UE_LOG output captured in trace)\n");
	fprintf(stderr, "  -logger=CsvFrames   Emit csv_frame_number records (CSV frame counter per game frame)\n");
	fprintf(stderr, "\n");
	fprintf(stderr, "Output: JSONL on stdout (one JSON object per line)\n");
}

static bool ParseArgs(int32 ArgC, TCHAR const* const* ArgV, FTraceQueryOptions& OutOptions)
{
	if (ArgC < 2)
	{
		PrintUsage();
		return false;
	}

	OutOptions.InputFile = ArgV[1];

	for (int32 ArgIndex = 2; ArgIndex < ArgC; ++ArgIndex)
	{
		FString Arg = ArgV[ArgIndex];

		if (Arg.StartsWith(TEXT("-logger=")))
		{
			OutOptions.Logger = Arg.Mid(8);
		}
		else if (Arg.StartsWith(TEXT("-event=")))
		{
			OutOptions.EventPattern = Arg.Mid(7);
		}
		else if (Arg.StartsWith(TEXT("-counter=")))
		{
			OutOptions.CounterFilter = Arg.Mid(9);
		}
		else if (Arg.StartsWith(TEXT("-timers=")))
		{
			OutOptions.TimerFilter = Arg.Mid(8);
		}
		else if (Arg == TEXT("-summary"))
		{
			OutOptions.bSummaryOnly = true;
		}
		else if (Arg.StartsWith(TEXT("-timeA=")) || Arg.StartsWith(TEXT("-startTime=")))
		{
			OutOptions.TimeA = FCString::Atod(*Arg.Mid(Arg.StartsWith(TEXT("-timeA=")) ? 7 : 11));
		}
		else if (Arg.StartsWith(TEXT("-timeB=")) || Arg.StartsWith(TEXT("-endTime=")))
		{
			OutOptions.TimeB = FCString::Atod(*Arg.Mid(Arg.StartsWith(TEXT("-timeB=")) ? 7 : 9));
		}
		else if (Arg.StartsWith(TEXT("-timeC=")))
		{
			OutOptions.TimeC = FCString::Atod(*Arg.Mid(7));
		}
		else if (Arg.StartsWith(TEXT("-timeD=")))
		{
			OutOptions.TimeD = FCString::Atod(*Arg.Mid(7));
		}
		else if (Arg.StartsWith(TEXT("-rule=")))
		{
			OutOptions.ExplicitRule = Arg.Mid(6);
		}
		else if (Arg.StartsWith(TEXT("-memtag=")))
		{
			OutOptions.MemoryTagFilter = Arg.Mid(8);
		}
		else if (Arg.StartsWith(TEXT("-queryFile=")))
		{
			OutOptions.QueryFile = Arg.Mid(11);
		}
		else
		{
			fprintf(stderr, "Warning: Unknown argument '%s'\n", TCHAR_TO_ANSI(*Arg));
		}
	}

	OutOptions.ResolveLoggerFlags();

	if (OutOptions.bWantCsvFrameNumbers && !OutOptions.TimerFilter.IsEmpty())
	{
		fprintf(stderr, "Warning: -timers filter is ignored in -logger=CsvFrames mode (only 'Frame' breadcrumb scopes are captured)\n");
	}
	if ((OutOptions.bWantMemoryTags || OutOptions.bWantMemoryTimeline || OutOptions.bWantAllocationTags || OutOptions.bWantPackageMemory)
		&& !OutOptions.TimerFilter.IsEmpty())
	{
		fprintf(stderr, "Warning: -timers filter is ignored with memory loggers (memory traces contain no CPU profiler data)\n");
	}

	if (!OutOptions.QueryFile.IsEmpty() && !ValidateQueryFileRules(OutOptions.QueryFile))
	{
		return false;
	}

	if (!OutOptions.ExplicitRule.IsEmpty() && !FindQueryRule(OutOptions.ExplicitRule))
	{
		fprintf(stderr, "Error: Unknown -rule value '%s'. Valid rules: aAf, afA, Aaf, aAfB, AaBf, AfB, AaB, AafB, aABf, AaBCf, AaBfC, aABfC, AaBCfD\n",
			TCHAR_TO_ANSI(*OutOptions.ExplicitRule));
		return false;
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

static TArray<FPackageMemoryQuery> ParseQueryFile(const FString& FilePath, bool& bOutError)
{
	bOutError = false;
	TArray<FPackageMemoryQuery> Queries;

	FString FileContent;
	if (!FFileHelper::LoadFileToString(FileContent, *FilePath))
	{
		fprintf(stderr, "Error: Cannot read query file '%s'\n", TCHAR_TO_ANSI(*FilePath));
		bOutError = true;
		return Queries;
	}

	TSharedPtr<FJsonValue> Root;
	TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(FileContent);
	if (!FJsonSerializer::Deserialize(Reader, Root) || !Root.IsValid())
	{
		fprintf(stderr, "Error: Failed to parse query file '%s'\n", TCHAR_TO_ANSI(*FilePath));
		bOutError = true;
		return Queries;
	}

	const TArray<TSharedPtr<FJsonValue>>* Array;
	if (!Root->TryGetArray(Array))
	{
		fprintf(stderr, "Error: Query file must be a JSON array\n");
		bOutError = true;
		return Queries;
	}

	int32 EntryIndex = 0;
	for (const TSharedPtr<FJsonValue>& Entry : *Array)
	{
		const TSharedPtr<FJsonObject>* ObjPtr;
		if (!Entry->TryGetObject(ObjPtr))
		{
			fprintf(stderr, "Warning: Query file entry %d is not an object, skipping\n", EntryIndex);
			++EntryIndex;
			continue;
		}
		const TSharedPtr<FJsonObject>& Obj = *ObjPtr;

		FPackageMemoryQuery Query;
		Obj->TryGetStringField(TEXT("id"), Query.Id);
		if (Query.Id.IsEmpty())
		{
			fprintf(stderr, "Error: Query file entry %d missing required 'id' field\n", EntryIndex);
			bOutError = true;
			return Queries;
		}

		FString RuleStr;
		if (!Obj->TryGetStringField(TEXT("rule"), RuleStr))
		{
			fprintf(stderr, "Error: Query '%s' (entry %d) missing required 'rule' field\n",
				TCHAR_TO_ANSI(*Query.Id), EntryIndex);
			bOutError = true;
			return Queries;
		}
		if (const EQueryRule* Found = FindQueryRule(RuleStr))
		{
			Query.Rule = *Found;
		}
		else
		{
			fprintf(stderr, "Error: Unknown rule '%s' in query '%s'. Valid rules: aAf, afA, Aaf, aAfB, AaBf, AfB, AaB, AafB, aABf, AaBCf, AaBfC, aABfC, AaBCfD\n",
				TCHAR_TO_ANSI(*RuleStr), TCHAR_TO_ANSI(*Query.Id));
			bOutError = true;
			return Queries;
		}

		if (!Obj->TryGetNumberField(TEXT("timeA"), Query.TimeA))
		{
			Obj->TryGetNumberField(TEXT("startTime"), Query.TimeA);
		}
		if (!Obj->TryGetNumberField(TEXT("timeB"), Query.TimeB))
		{
			Obj->TryGetNumberField(TEXT("endTime"), Query.TimeB);
		}
		Obj->TryGetNumberField(TEXT("timeC"),     Query.TimeC);
		Obj->TryGetNumberField(TEXT("timeD"),     Query.TimeD);

		Queries.Add(Query);
		++EntryIndex;
	}

	return Queries;
}

enum class ETraceQueryExit : int32
{
	Success        = 0,
	BadArgs        = 1,
	FileOpenFailed = 2,
	QueryFileBad   = 3,
};

static int32 TraceQueryMain(int32 ArgC, TCHAR const* const* ArgV)
{
	FTraceQueryOptions Options;
	if (!ParseArgs(ArgC, ArgV, Options))
	{
		return (int32)ETraceQueryExit::BadArgs;
	}

	UE::Trace::FFileDataStream DataStream;
	if (!DataStream.Open(*Options.InputFile))
	{
		fprintf(stderr, "Error: Cannot open trace file '%s'\n", TCHAR_TO_ANSI(*Options.InputFile));
		return (int32)ETraceQueryExit::FileOpenFailed;
	}

	{
		FJsonTraceAnalyzer Analyzer(Options);

		UE::Trace::FAnalysisContext AnalysisContext;
		AnalysisContext.AddAnalyzer(Analyzer);

		// Create an analysis session (needed by TraceServices factory analyzers for string
		// interning, edit scopes, etc.). We drive analysis ourselves via FAnalysisContext::Process,
		// so the session doesn't need a data stream.
		TSharedPtr<TraceServices::IAnalysisSession> Session =
			TraceServices::CreateAnalysisSession(0, nullptr, TUniquePtr<UE::Trace::IInDataStream>(nullptr));

		// CPU profiler via TraceServices factory analyzer
		TSharedPtr<UE::Trace::IAnalyzer> CpuProfilerAnalyzer;
		FTraceQueryCpuProfilerProvider CpuProfilerProvider;
		if (Options.bWantCpuProfiler || Options.bWantAllEvents)
		{
			CpuProfilerProvider.Options = &Options;
			CpuProfilerProvider.ScopeGroups = &Analyzer.ScopeGroups;
			CpuProfilerProvider.NumCpuSpecs = &Analyzer.NumCpuSpecs;
			CpuProfilerProvider.NumCpuEvents = &Analyzer.NumCpuEvents;
			if (Options.bWantCsvFrameNumbers)
			{
				CpuProfilerProvider.CsvFrameNumbers = &Analyzer.CsvFrameNumbers;
			}
			Analyzer.CpuProfilerProvider = &CpuProfilerProvider;
			CpuProfilerAnalyzer = TraceServices::CreateCpuProfilerAnalyzer(
				*Session, CpuProfilerProvider, CpuProfilerProvider);
			AnalysisContext.AddAnalyzer(*CpuProfilerAnalyzer);
		}

		// Counters via TraceServices factory analyzer
		TSharedPtr<UE::Trace::IAnalyzer> CountersAnalyzer;
		FTraceQueryCounterProvider CountersProvider;
		if (Options.bWantCounters || Options.bWantAllEvents)
		{
			CountersProvider.Options = &Options;
			CountersProvider.EmitCallback = [&Analyzer](const char* Line) { Analyzer.EmitJsonLine(Line); };
			CountersProvider.TimeFilterCallback = [&Analyzer](double T) { return Analyzer.PassesTimeFilter(T); };
			CountersProvider.JsonEscapeCallback = [](const FString& S) { return FJsonTraceAnalyzer::JsonEscape(S); };
			CountersProvider.NumCounterSpecs = &Analyzer.NumCounterSpecs;
			CountersProvider.NumCounterValues = &Analyzer.NumCounterValues;
			CountersAnalyzer = TraceServices::CreateCountersAnalyzer(*Session, CountersProvider);
			AnalysisContext.AddAnalyzer(*CountersAnalyzer);
		}

		// Memory via TraceServices factory analyzers
		// Not registered with Session -- consumed directly by FMemoryAnalyzer (no Session lookup needed)
		TraceServices::FMemoryProvider MemProvider(*Session);
		TSharedPtr<UE::Trace::IAnalyzer> MemoryTagAnalyzer;
		if (Options.bWantMemoryTags)
		{
			MemoryTagAnalyzer = MakeMemoryAnalyzer(*Session, MemProvider);
			AnalysisContext.AddAnalyzer(*MemoryTagAnalyzer);
		}
		TSharedPtr<TraceServices::FDefinitionProvider> DefProvider;
		TSharedPtr<UE::Trace::IAnalyzer> StringsAnalyzer;
		if (Options.bWantPackageMemory)
		{
			// FStringsAnalyzer retrieves IDefinitionProvider via session lookup; register it explicitly.
			DefProvider = MakeShared<TraceServices::FDefinitionProvider>(Session.Get());
			Session->AddProvider(TraceServices::GetDefinitionProviderName(), DefProvider, DefProvider);
			StringsAnalyzer = MakeShared<TraceServices::FStringsAnalyzer>(*Session);
			AnalysisContext.AddAnalyzer(*StringsAnalyzer);
		}
		// Not registered with Session -- consumed directly by FMetadataAnalysis and FAllocationsAnalyzer
		TraceServices::FMetadataProvider MetaProvider(*Session);
		TSharedPtr<TraceServices::FAllocationsProvider> AllocsProvider;
		TSharedPtr<UE::Trace::IAnalyzer> MetadataAnalyzer;
		TSharedPtr<UE::Trace::IAnalyzer> AllocationsAnalyzer;
		if (Options.bWantMemoryTimeline || Options.bWantAllocationTags || Options.bWantPackageMemory)
		{
			MetadataAnalyzer = MakeMetadataAnalyzer(*Session, MetaProvider);
			AnalysisContext.AddAnalyzer(*MetadataAnalyzer);
			AllocsProvider = MakeShared<TraceServices::FAllocationsProvider>(*Session, MetaProvider);
			AllocationsAnalyzer = MakeAllocationsAnalyzer(*Session, *AllocsProvider, MetaProvider);
			AnalysisContext.AddAnalyzer(*AllocationsAnalyzer);
		}

		// Regions
		// Not registered with Session -- consumed directly by FRegionAnalyzer
		TraceServices::FRegionProvider RegionProvider(*Session);
		FRegionAnalyzer RegionAnalyzer(*Session, RegionProvider);
		if (Options.bWantRegions || Options.bWantAllEvents)
		{
			AnalysisContext.AddAnalyzer(RegionAnalyzer);
		}

		// Log via TraceServices factory analyzer
		TSharedPtr<TraceServices::FLogProvider> LogProvider;
		TSharedPtr<UE::Trace::IAnalyzer> LogAnalyzer;
		if (Options.bWantLog || Options.bWantAllEvents)
		{
			LogProvider = MakeShared<TraceServices::FLogProvider>(*Session);
			LogAnalyzer = MakeLogAnalyzer(*Session, *LogProvider);
			AnalysisContext.AddAnalyzer(*LogAnalyzer);
		}

		AnalysisContext.Process(DataStream).Wait();

		if (Options.bWantMemoryTags)
		{
			if (MemProvider.IsInitialized())
			{
				Analyzer.EmitMemoryTags(MemProvider);
			}
		}
		if (Options.bWantRegions || Options.bWantAllEvents)
		{
			Analyzer.EmitRegions(RegionProvider);
		}
		if ((Options.bWantLog || Options.bWantAllEvents) && LogProvider)
		{
			Analyzer.EmitLogMessages(*LogProvider);
		}
		if (Options.bWantMemoryTimeline || Options.bWantAllocationTags || Options.bWantPackageMemory)
		{
			if (AllocsProvider && AllocsProvider->IsInitialized())
			{
				// MemoryTimeline and AllocationTags require actual allocation events
				if (AllocsProvider->HasAllocationEvents())
				{
					if (Options.bWantMemoryTimeline)
					{
						Analyzer.EmitMemoryTimeline(*AllocsProvider);
					}
					if (Options.bWantAllocationTags)
					{
						Analyzer.EmitAllocationTags(*AllocsProvider);
					}
				}
				// PackageMemory reads from allocation metadata, which may exist even
				// when HasAllocationEvents() is false (e.g. LWM traces)
				if (Options.bWantPackageMemory)
				{
					if (DefProvider)
					{
						if (!Options.QueryFile.IsEmpty())
						{
							// Batch mode: parse query file and execute each entry in sequence.
							// Analysis ran once; all queries share the in-memory SbTree.
							bool bQueryFileError = false;
							const TArray<FPackageMemoryQuery> BatchQueries = ParseQueryFile(Options.QueryFile, bQueryFileError);
							if (bQueryFileError)
							{
								return (int32)ETraceQueryExit::QueryFileBad;
							}
							for (const FPackageMemoryQuery& Query : BatchQueries)
							{
								Analyzer.EmitPackageMemory(*AllocsProvider, MetaProvider, *DefProvider, Query);
							}
						}
						else
						{
							// Single-query mode: build params from command-line flags (backward compatible).
							FPackageMemoryQuery Query;
							if (!Options.ExplicitRule.IsEmpty())
							{
								const EQueryRule* Found = FindQueryRule(Options.ExplicitRule);
								if (!Found)
								{
									fprintf(stderr, "Error: Unknown rule '%s'. Valid rules: aAf, afA, Aaf, aAfB, AaBf, AfB, AaB, AafB, aABf, AaBCf, AaBfC, aABfC, AaBCfD\n",
										TCHAR_TO_ANSI(*Options.ExplicitRule));
									return (int32)ETraceQueryExit::BadArgs;
								}
								Query.Rule  = *Found;
								Query.TimeA = (Options.TimeA >= 0.0) ? Options.TimeA : 0.0;
								Query.TimeB = (Options.TimeB >= 0.0) ? Options.TimeB : 0.0;
								Query.TimeC = Options.TimeC;
								Query.TimeD = Options.TimeD;
							}
							else if (Options.TimeA >= 0.0 && Options.TimeB >= 0.0)
							{
								Query.Rule  = EQueryRule::AaBf;
								Query.TimeA = Options.TimeA;
								Query.TimeB = Options.TimeB;
							}
							else if (Options.TimeA >= 0.0)
							{
								Query.Rule  = EQueryRule::Aaf;
								Query.TimeA = Options.TimeA;
							}
							else if (Options.TimeB >= 0.0)
							{
								Query.Rule  = EQueryRule::aAf;
								Query.TimeA = Options.TimeB;
							}
							else
							{
								Query.Rule  = EQueryRule::Aaf;
								Query.TimeA = 0.0;
							}
							Analyzer.EmitPackageMemory(*AllocsProvider, MetaProvider, *DefProvider, Query);
						}
					}
				}
			}
		}
	}

	RequestEngineExit(TEXT("TraceQuery completed"));
	return (int32)ETraceQueryExit::Success;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

INT32_MAIN_INT32_ARGC_TCHAR_ARGV()
{
	FCommandLine::Set(TEXT(""));
	return TraceQueryMain(ArgC, ArgV);
}
