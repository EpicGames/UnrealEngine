// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/UnrealString.h"
#include "Logging/LogCategory.h"

DECLARE_LOG_CATEGORY_EXTERN(LogTraceQuery, Log, All);

/**
 * Options parsed from the command line that control which trace events are decoded
 * and how the JSONL output is filtered and formatted.
 */
struct FTraceQueryOptions
{
	FString InputFile;

	// Filter by logger name (e.g., "Stats", "Counters", "CpuProfiler")
	FString Logger;

	// Filter by event name pattern (* wildcard)
	FString EventPattern;

	// Filter counters by name substring (matches both Stats-sourced and Counters-channel events)
	FString CounterFilter;

	// Filter CPU profiler scopes by name substring
	FString TimerFilter;

	// Time markers. TimeA/TimeB double as the generic start/end time filter for
	// non-MemoryPackages loggers. -startTime=/-endTime= are accepted as aliases.
	double TimeA = -1.0;
	double TimeB = -1.0;
	double TimeC = 0.0;
	double TimeD = 0.0;

	// Explicit query rule for -logger=MemoryPackages single-query mode.
	// Empty = auto-derive from TimeA/TimeB (backward compatible).
	FString ExplicitRule;

	// Print event type summary to stdout, then exit
	bool bSummaryOnly = false;

	// Whether to include specific loggers (derived from -logger and filter flags)
	bool bWantStats = true;
	bool bWantCounters = true;
	bool bWantCpuProfiler = true;  // Enabled by default  --  cycle stats live in CpuProfiler
	bool bWantAllEvents = false;

	// Memory logger flags (mutually exclusive with CPU/stats loggers)
	bool bWantMemoryTags = false;       // emit LLM tag names and time-series sample values
	bool bWantMemoryTimeline = false;   // emit total-allocated-memory timeline for spike detection
	bool bWantAllocationTags = false;   // emit per-package/asset allocation tag names (from IAllocationsProvider)
	bool bWantPackageMemory = false;    // emit per-package memory sizes (from allocation metadata "Asset" field)

	// Regions logger flag
	bool bWantRegions = false;          // emit region records (TRACE_BEGIN_REGION / TRACE_END_REGION)

	// Log logger flag
	bool bWantLog = false;              // emit log_msg records (UE_LOG output captured in trace)

	// CsvFrames logger flag
	bool bWantCsvFrameNumbers = false;  // emit csv_frame_number records (CSV frame counter per game frame)

	// Filter: only emit LLM tags whose name contains this substring (case-insensitive)
	FString MemoryTagFilter;            // -memtag=<filter>

	// Path to a JSON file containing multiple MemoryPackages query entries.
	// When set with -logger=MemoryPackages, all queries run in one TraceQuery process.
	// Individual -timeA / -timeB (and their -startTime/-endTime aliases) are ignored when this is set.
	FString QueryFile;              // -queryFile=<path>

	void ResolveLoggerFlags()
	{
		if (!Logger.IsEmpty())
		{
			// If a specific logger is requested, only enable that one
			bWantStats = Logger.Equals(TEXT("Stats"), ESearchCase::IgnoreCase);
			bWantCounters = Logger.Equals(TEXT("Counters"), ESearchCase::IgnoreCase);
			bWantCpuProfiler = Logger.Equals(TEXT("CpuProfiler"), ESearchCase::IgnoreCase);
			bWantAllEvents = Logger.Equals(TEXT("*"), ESearchCase::IgnoreCase);

			bWantMemoryTags     = Logger.Equals(TEXT("MemoryTags"),     ESearchCase::IgnoreCase)
			                   || Logger.Equals(TEXT("Memory"),          ESearchCase::IgnoreCase);
			bWantMemoryTimeline = Logger.Equals(TEXT("MemoryTimeline"), ESearchCase::IgnoreCase)
			                   || Logger.Equals(TEXT("Memory"),          ESearchCase::IgnoreCase);
			bWantAllocationTags = Logger.Equals(TEXT("MemoryAssets"),    ESearchCase::IgnoreCase)
			                   || Logger.Equals(TEXT("Memory"),          ESearchCase::IgnoreCase);
			bWantPackageMemory  = Logger.Equals(TEXT("MemoryPackages"),  ESearchCase::IgnoreCase)
			                   || Logger.Equals(TEXT("Memory"),          ESearchCase::IgnoreCase);

			bWantRegions        = Logger.Equals(TEXT("Regions"),        ESearchCase::IgnoreCase);

			bWantLog            = Logger.Equals(TEXT("Log"),            ESearchCase::IgnoreCase);

			bWantCsvFrameNumbers = Logger.Equals(TEXT("CsvFrames"),     ESearchCase::IgnoreCase);

			// -logger=Stats should also enable CpuProfiler since cycle stats are there
			if (bWantStats)
			{
				bWantCpuProfiler = true;
			}

			// -logger=CsvFrames needs CpuProfiler for Frame scope metadata
			if (bWantCsvFrameNumbers)
			{
				bWantCpuProfiler = true;
			}
		}

		// -counter filter implies we want both Stats batches AND Counters channel
		if (!CounterFilter.IsEmpty())
		{
			bWantStats = true;
			bWantCounters = true;
		}

		// -timers filter implies we want CpuProfiler
		if (!TimerFilter.IsEmpty())
		{
			bWantCpuProfiler = true;
		}

		// Memory traces don't contain CPU profiler / counters / stats data
		if (bWantMemoryTags || bWantMemoryTimeline || bWantAllocationTags || bWantPackageMemory)
		{
			bWantStats = bWantCounters = bWantCpuProfiler = bWantAllEvents = false;
		}
	}
};
