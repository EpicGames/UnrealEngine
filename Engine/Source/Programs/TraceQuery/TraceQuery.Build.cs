// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TraceQuery : ModuleRules
{
	public TraceQuery(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePathModuleNames.Add("Launch");
		PrivateDependencyModuleNames.Add("Core");
		PrivateDependencyModuleNames.Add("Projects");
		PrivateDependencyModuleNames.Add("TraceAnalysis");
		PrivateDependencyModuleNames.Add("TraceServices");
		PrivateDependencyModuleNames.Add("Json");

		// TODO: Replace this private include path with public factory functions in TraceServices::AnalyzerFactories.h.
		// Required for: FMemoryProvider, FMetadataProvider, FAllocationsProvider, FLogProvider, and their analyzers.
		// The Insights team has noted this requires altering constructors to accept interface classes, which is a
		// non-trivial change. Making local copies of classes would involve cloning 30 files.
		// Until then, internal TraceServices refactors can silently break TraceQuery with no
		// signal to the Insights team. Add a ticket here when tracked: <ticket>
		PrivateIncludePaths.Add(System.IO.Path.Combine(ModuleDirectory, "../../Developer/TraceServices/Private"));
	}
}
